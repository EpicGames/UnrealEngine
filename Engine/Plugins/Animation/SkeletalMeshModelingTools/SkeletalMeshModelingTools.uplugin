{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Skeletal Mesh Editing Tools",
	"Description": "Create skeletons, paint skin weights and edit skeletal meshes.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com/",
	"DocsURL": "",
	"MarketplaceURL": "",
	"EnabledByDefault" : true,
	"CanContainContent": true,
	"IsExperimentalVersion": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SkeletalMeshModelingTools",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "GeometryScripting",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}
