// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SKMBackedDynaMeshComponent.h"

#include "DynamicMeshToMeshDescription.h"
#include "DynamicSubmesh3.h"
#include "GroupTopology.h"
#include "LODUtilities.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "ModelingToolTargetEditorOnlyUtil.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletonModifier.h"
#include "SkeletalMeshTypes.h"
#include "Misc/ITransaction.h"
#include "Editor.h"
#include "Editor/TransBuffer.h"
#include "SkeletalMeshEditorSubsystem.h"
#include "Parameterization/MeshPlanarSymmetry.h"
#include "ToolTargets/SkeletalMeshToolTarget.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SKMBackedDynaMeshComponent)

#define LOCTEXT_NAMESPACE "SKMBackedDynaMeshComponent"

void USkeletalMeshBackedDynamicMeshComponent::FSkeletonChangeTracker::Init(int32 NumBones)
{
	BoneIndexTracker.Reserve(NumBones);
	for (int32 Index = 0; Index < NumBones; Index++)
	{
		BoneIndexTracker.Add(Index);
	}

	ChangeCount = 0;
}

void USkeletalMeshBackedDynamicMeshComponent::FSkeletonChangeTracker::HandleSkeletonChanged(const TArray<int32> ToolBoneIndexTracker)
{
	for (int32 BoneIndex = 0; BoneIndex < BoneIndexTracker.Num(); BoneIndex++)
	{
		int32 PreviousNewBoneIndex = BoneIndexTracker[BoneIndex];
		if (ToolBoneIndexTracker.IsValidIndex(PreviousNewBoneIndex))
		{
			int32 NewBoneIndex = ToolBoneIndexTracker[PreviousNewBoneIndex];
			BoneIndexTracker[BoneIndex] = NewBoneIndex;
		}
	}

	ChangeCount++;
}

int32 USkeletalMeshBackedDynamicMeshComponent::FSkeletonChangeTracker::GetChangeCount() const
{
	return ChangeCount;
}

const TArray<int32>& USkeletalMeshBackedDynamicMeshComponent::FSkeletonChangeTracker::GetBoneIndexTracker() const
{
	return BoneIndexTracker;
}

void USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::Init(const TArray<FName> ExistingMorphTargets)
{
	OriginalNameToCurrentName.Reset();
	CurrentNameToOriginalName.Reset();
	for (const FName& MorphTarget : ExistingMorphTargets)
	{
		OriginalNameToCurrentName.Emplace(MorphTarget, MorphTarget);
		CurrentNameToOriginalName.Emplace(MorphTarget, MorphTarget);
	}
}

void USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::HandleRenameMorphTarget(FName CurrentName, FName NewName)
{
	FName OriginalName = CurrentNameToOriginalName[CurrentName];

	if (OriginalName != NAME_None)
	{
		OriginalNameToCurrentName[OriginalName] = NewName;
	}

	CurrentNameToOriginalName.Remove(CurrentName);
	CurrentNameToOriginalName.Emplace(NewName, OriginalName);

	if (EditedMorphTargets.Contains(CurrentName))
	{
		EditedMorphTargets.Remove(CurrentName);
		EditedMorphTargets.Add(NewName);
	}
}

void USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::HandleRemoveMorphTarget(FName Name)
{
	FName OriginalName = CurrentNameToOriginalName[Name];
	if (OriginalName != NAME_None)
	{
		OriginalNameToCurrentName[OriginalName] = NAME_None;
	}

	CurrentNameToOriginalName.Remove(Name);
	
	EditedMorphTargets.Remove(Name);
}

void USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::HandleAddMorphTarget(FName Name)
{
	CurrentNameToOriginalName.Emplace(Name, NAME_None);
}

void USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::HandleEditMorphTarget(FName Name)
{
	EditedMorphTargets.Add(Name);
}

FName USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::GetCurrentMorphTargetName(FName OriginalName) const
{
	if (const FName* CurrentName = OriginalNameToCurrentName.Find(OriginalName))
	{
		return *CurrentName;
	}

	return NAME_None;
}

FName USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::GetOriginalMorphTargetName(FName CurrentName) const
{
	if (const FName* OriginalName = CurrentNameToOriginalName.Find(CurrentName))
	{
		return *OriginalName;
	}

	return NAME_None;
}

const TSet<FName>& USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::GetEditedMorphTargets() const
{
	return EditedMorphTargets;
}

TArray<USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::FNameInfo> USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::GetCurvesToRename() const
{
	TArray<FNameInfo> Renamed;
	for (const TPair<FName, FName>& OriginalToCurrent : OriginalNameToCurrentName)
	{
		if (OriginalToCurrent.Value != OriginalToCurrent.Key && OriginalToCurrent.Value != NAME_None)
		{
			FNameInfo Info;
			Info.OldName = OriginalToCurrent.Key;
			Info.NewName = OriginalToCurrent.Value;
			Renamed.Add(Info);
		}
	}

	return Renamed;
}

TArray<FName> USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::GetCurvesToRemove() const
{
	TArray<FName> Removed;
	for (const TPair<FName, FName>& OriginalToCurrent : OriginalNameToCurrentName)
	{
		if (OriginalToCurrent.Value == NAME_None)
		{
			Removed.Add(OriginalToCurrent.Key);
		}
	}

	return Removed;
}

TArray<FName> USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::GetCurvesToAdd() const
{
	TArray<FName> Added;	
	for (const TPair<FName, FName>& CurrentToOriginal : CurrentNameToOriginalName)
	{
		if (CurrentToOriginal.Value == NAME_None)
		{
			Added.Add(CurrentToOriginal.Key);
		}
	}
	
	return Added;
}

TArray<FName> USkeletalMeshBackedDynamicMeshComponent::FMorphTargetChangeTracker::GetCurrentMorphTargetNames() const
{
	TArray<FName> CurrentMorphTargetNames;
	CurrentNameToOriginalName.GenerateKeyArray(CurrentMorphTargetNames);
	return CurrentMorphTargetNames;
}

EMeshLODIdentifier USkeletalMeshBackedDynamicMeshComponent::Init(USkeletalMesh* InSkeletalMesh, EMeshLODIdentifier InLOD)
{
	WeakSkeletalMesh = InSkeletalMesh;
	LOD = GetValidEditingLOD(InLOD);
	LODIndex = static_cast<int32>(InLOD);

	// Mesh
	FMeshDescription* MeshDescription = InSkeletalMesh->GetMeshDescription(LODIndex);
	FDynamicMesh3 LODMesh;
	FMeshDescriptionToDynamicMesh Converter;
	// By default, when converting dyna mesh back to mesh description, we don't transform vert color, so avoid doing it in the first place.
	// See FDynamicMeshCommitInfo
	Converter.bTransformVertexColorsLinearToSRGB = false;
	constexpr bool bWantTangents = true;
	Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
	Converter.Convert(MeshDescription, LODMesh, bWantTangents);

	SetMesh(MoveTemp(LODMesh));

	// Additional Trackers
	ResetTrackersDirect();

	// Subscribe to transaction-state changes so we can flush pending change events the moment
	// the enclosing transaction settles, instead of relying on the editor mode's Tick. This
	// keeps observers (e.g. USkeletalMeshEditingCache::RefSkeletonPoser) coherent across tool
	// switches, which happen between frames.
	if (GEditor && GEditor->Trans)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			// Support for calling init multiple times is future work ( could be useful for keeping undo history across LOD changes)
			if (ensure(!TransBuffer->OnTransactionStateChanged().IsBoundToObject(this)))
			{
				TransactionStateChangedHandle = TransBuffer->OnTransactionStateChanged().AddUObject(
					this, &USkeletalMeshBackedDynamicMeshComponent::OnTransactionStateChanged);
			}
		}
	}

	return static_cast<EMeshLODIdentifier>(LODIndex);
}

EMeshLODIdentifier USkeletalMeshBackedDynamicMeshComponent::GetLOD() const
{
	return LOD;
}

bool USkeletalMeshBackedDynamicMeshComponent::CommitToSkeletalMesh()
{
	if (GetChangeCount() == 0)
	{
		return false;
	}

	// If we are expecting a asset change, it means a commit is already in progress
	if (bExpectAssetChange)
	{
		return false;
	}

	bExpectAssetChange = true;
	
	USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get();

	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);

		TArray<FName> OldEngineGeneratedMorphTargets;
		USkeletalMeshEditorSubsystem::GetMorphTargetsGeneratedByEngine(SkeletalMesh, OldEngineGeneratedMorphTargets);

		if (GetSkeletonChangeTracker().GetChangeCount() != 0)
		{
			USkeletonModifier* SkeletonModifier = NewObject<USkeletonModifier>();
			SkeletonModifier->SetSkeletalMesh(SkeletalMesh);

			SkeletonModifier->ExternalUpdate(GetRefSkeleton(), GetSkeletonChangeTracker().GetBoneIndexTracker());
			SkeletonModifier->CommitSkeletonToSkeletalMesh();
		}

		USkeletalMeshToolTargetFactory* LocalFactory = NewObject<USkeletalMeshToolTargetFactory>();
		LocalFactory->SetActiveEditingLOD(GetLOD());
		USkeletalMeshToolTarget* LocalTarget = CastChecked<USkeletalMeshToolTarget>(LocalFactory->BuildTarget(SkeletalMesh, {}));

		LocalTarget->CommitDynamicMesh(*GetMesh());

		// Make sure morph targets are marked as engine generated correctly
		// As soon as a morph is edited in engine, we want to mark it such that reimports in the future don't overwrite our edits.
		{
			TArray<FString> NewEngineGeneratedMorphTargets;
	
			for (const FName& OldEngineGeneratedMorphTarget : OldEngineGeneratedMorphTargets)
			{
				FName CurrentName = GetMorphTargetChangeTracker().GetCurrentMorphTargetName(OldEngineGeneratedMorphTarget);
				if (CurrentName != NAME_None)
				{
					NewEngineGeneratedMorphTargets.AddUnique(CurrentName.ToString());
					USkeletalMeshEditorSubsystem::SetMorphTargetsToGeneratedByEngine(SkeletalMesh, {CurrentName.ToString()});
				}
			}
	
			for (const FName& EditedMorphTarget : GetMorphTargetChangeTracker().GetEditedMorphTargets())
			{
				NewEngineGeneratedMorphTargets.AddUnique(EditedMorphTarget.ToString());	
			}
	
			USkeletalMeshEditorSubsystem::SetMorphTargetsToGeneratedByEngine(SkeletalMesh, NewEngineGeneratedMorphTargets);
		}

		// Also change related curves
		{
		
			UAnimCurveMetaData* AnimCurveMetaData = SkeletalMesh->GetAssetUserData<UAnimCurveMetaData>();
			if (AnimCurveMetaData == nullptr)
			{
				AnimCurveMetaData = NewObject<UAnimCurveMetaData>(SkeletalMesh, NAME_None, RF_Transactional);
				SkeletalMesh->AddAssetUserData(AnimCurveMetaData);
			}

			for (const FMorphTargetChangeTracker::FNameInfo& NameInfo : GetMorphTargetChangeTracker().GetCurvesToRename())
			{
				// Only rename if we have a morph flag set
				if (FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(NameInfo.OldName))
				{
					if (CurveMetaData->Type.bMorphtarget)
					{
						AnimCurveMetaData->RenameCurveMetaData(NameInfo.OldName, NameInfo.NewName);
					}
				}	
			}
	
			for (const FName OldCurve : GetMorphTargetChangeTracker().GetCurvesToRemove())
			{
				// Only remove if we have a morph flag set
				if (FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(OldCurve))
				{
					if (CurveMetaData->Type.bMorphtarget)
					{
						AnimCurveMetaData->RemoveCurveMetaData(OldCurve);
					}
				}	
			}
	
			for (const FName NewCurve : GetMorphTargetChangeTracker().GetCurvesToAdd())
			{
				AnimCurveMetaData->AddCurveMetaData(NewCurve);
		
				// Ensure we have a morph flag set
				if (FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(NewCurve))
				{
					AnimCurveMetaData->Modify();
					CurveMetaData->Type.bMorphtarget = true;
				}	
			}
		}

		{
			FTrackerChangeScope TrackerScope(this);
			TrackerScope.MarkAsCommitChange();
			ResetTrackersDirect();
		}
		
	} // FScopedSkeletalMeshPostEditChange exits, Skeletal Mesh build triggered
	
	return true;
}

void USkeletalMeshBackedDynamicMeshComponent::DiscardChanges()
{
	// Records a change that turns current mesh back to the initial state
	FChangeScope ChangeScope(this);
	
	FDynamicMesh3 InitialMeshCopy = *Trackers.InitialAssetMesh;
	SetMesh(MoveTemp(InitialMeshCopy));

	ResetTrackersDirect();
}


void USkeletalMeshBackedDynamicMeshComponent::HandleSkeletonChange(USkeletonModifier* InModifier)
{
	FTrackerChangeScope ChangeScope(this);
	
	Trackers.RefSkeleton = InModifier->GetReferenceSkeleton();
	Trackers.RefSkeleton.GetBoneAbsoluteTransforms(Trackers.ComponentSpaceBoneTransformsRefPose);
	Trackers.SkeletonChangeTracker.HandleSkeletonChanged(InModifier->GetBoneIndexTracker());
	MarkDirtyDirect();
}

void USkeletalMeshBackedDynamicMeshComponent::ForwardVisibilityChangeRequest(bool bInVisible)
{
	// This component is typically hidden as it only serves as data container
	// A separate mesh should be used to preview the skeletal mesh represented by this component,
	// which should receive visibility updates instead.
	OnRequestingVisibilityChangeDelegate.Broadcast(bInVisible);
}

void USkeletalMeshBackedDynamicMeshComponent::HandleGeometryUpdate(
	const FName EditingMorphTargetName,
	const FDynamicMesh3& PosedMesh,
	const TArray<FMatrix>& BoneMatrices,
	const TMap<FName, float>& MorphTargetWeights,
	const UE::Geometry::FDynamicSubmesh3* InSubmesh)
{
	using namespace UE::Geometry;
	using namespace SkeletalMeshToolsHelper;

	FDynamicMeshMorphTargetAttribute* MorphTargetAttribute = nullptr;
	float EditingMorphTargetWeight = 1.0f;
	
	ProcessMesh([&](const FDynamicMesh3& Mesh)
		{
			MorphTargetAttribute = Mesh.Attributes()->GetMorphTargetAttribute(EditingMorphTargetName);
			if (MorphTargetAttribute)
			{
				const float* FoundWeight = MorphTargetWeights.Find(EditingMorphTargetName);
				if (ensure(FoundWeight))
				{
					EditingMorphTargetWeight = *FoundWeight;
				}
			}
		});

	if (FMath::IsNearlyZero(EditingMorphTargetWeight))
	{
		return;
	}
	
	FChangeScope ChangeScope(this);
	
	// Mesh change
	EditMesh([&](FDynamicMesh3& Mesh)
		{
			TFunction<void(FVertInfo, const FVector&)> WriteFunction;
			TMap<FName, float> UnposeMorphTargetWeights = MorphTargetWeights;

			if (MorphTargetAttribute)
			{
				WriteFunction = [&](FVertInfo VertInfo, const FVector& UnposedVertPos)
				{
					const int32 VertID = VertInfo.VertID;
					FVector NewDelta = UnposedVertPos - Mesh.GetVertexRef(VertID);
					NewDelta /= EditingMorphTargetWeight;
					MorphTargetAttribute->SetValue(VertID, NewDelta);
				};

				// Exclude the morph target we are trying to extract
				UnposeMorphTargetWeights.Remove(EditingMorphTargetName);
			}
			else
			{
				WriteFunction = [&](FVertInfo VertInfo, const FVector& UnposedVertPos)
				{
					Mesh.SetVertex(VertInfo.VertID, UnposedVertPos);
				};
			}

			// SourceMesh = base mesh always. The submesh's attribute snapshot (skin weights + every
			// morph delta) is taken at SetTriangleIsolation time and never refreshed; using it here
			// makes the unpose math diverge from DeformPreviewMesh's pose math after any edit to a
			// different morph target, contaminating the editing morph delta with the divergence.
			// When isolated, the PosedMesh lives in submesh-VID space, so we translate per-vert.
			TArray<int32> BaseVertArray;
			if (InSubmesh)
			{
				const FDynamicMesh3& Submesh = InSubmesh->GetSubmesh();
				BaseVertArray.Reserve(Submesh.VertexCount());
				for (int32 SubVID : Submesh.VertexIndicesItr())
				{
					BaseVertArray.Add(InSubmesh->MapVertexToBaseMesh(SubVID));
				}
			}

			auto GetPosedVertexFunc = [&PosedMesh, InSubmesh](int32 BaseVID) -> FVector
			{
				const int32 PosedVID = InSubmesh ? InSubmesh->MapVertexToSubmesh(BaseVID) : BaseVID;
				return PosedMesh.GetVertex(PosedVID);
			};

			GetUnposedMesh(WriteFunction, GetPosedVertexFunc, Mesh, BoneMatrices, NAME_None, UnposeMorphTargetWeights, BaseVertArray);
		});
			
	// Tracker change
	if (MorphTargetAttribute)
	{
		Trackers.MorphTargetChangeTracker.HandleEditMorphTarget(EditingMorphTargetName);
	}
	else
	{
		MarkBaseGeometryDirtyDirect();
	}
	
	MarkDirtyDirect();
}

FName USkeletalMeshBackedDynamicMeshComponent::AddMorphTargetDirect(FDynamicMesh3& Mesh, FName InName)
{
	using namespace UE::Geometry;
	const FName ActualName = GetAvailableMorphTargetName(InName);

	FDynamicMeshMorphTargetAttribute* MorphTargetAttribute = new FDynamicMeshMorphTargetAttribute(&Mesh);
	MorphTargetAttribute->SetName(ActualName);
	Mesh.Attributes()->AttachMorphTargetAttribute(ActualName, MorphTargetAttribute);

	Trackers.MorphTargetChangeTracker.HandleAddMorphTarget(ActualName);

	return ActualName;
}

FName USkeletalMeshBackedDynamicMeshComponent::AddMorphTarget(FName InName)
{
	FChangeScope ChangeScope(this);
	FName Result = NAME_None;
	EditMesh([&](FDynamicMesh3& Mesh) { Result = AddMorphTargetDirect(Mesh, InName); });
	MarkDirtyDirect();
	return Result;
}

TArray<FName> USkeletalMeshBackedDynamicMeshComponent::AddMorphTargetsIfMissing(const TArray<FName>& Names)
{
	const TArray<FName>& Existing = GetMorphTargetChangeTracker().GetCurrentMorphTargetNames();

	TArray<FName> ToCreate;
	ToCreate.Reserve(Names.Num());
	for (const FName& Name : Names)
	{
		if (!Name.IsNone() && !Existing.Contains(Name) && !ToCreate.Contains(Name))
		{
			ToCreate.Add(Name);
		}
	}
	if (ToCreate.IsEmpty())
	{
		return {};
	}

	TArray<FName> Created;
	Created.Reserve(ToCreate.Num());

	FChangeScope ChangeScope(this);
	EditMesh([&](FDynamicMesh3& Mesh)
	{
		for (const FName& Name : ToCreate)
		{
			Created.Add(AddMorphTargetDirect(Mesh, Name));
		}
	});
	MarkDirtyDirect();

	return Created;
}

FName USkeletalMeshBackedDynamicMeshComponent::RenameMorphTargetDirect(FDynamicMesh3& Mesh, FName InOldName, FName InNewName)
{
	using namespace UE::Geometry;
	if (InOldName == InNewName)
	{
		return InNewName;
	}
	if (!GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(InOldName))
	{
		return NAME_None;
	}

	const FName ActualNewName = GetAvailableMorphTargetName(InNewName, InOldName);

	FDynamicMeshMorphTargetAttribute* OldMorphTargetAttribute = Mesh.Attributes()->GetMorphTargetAttribute(InOldName);

	FDynamicMeshMorphTargetAttribute* NewMorphTargetAttribute = new FDynamicMeshMorphTargetAttribute(&Mesh);
	NewMorphTargetAttribute->SetName(ActualNewName);
	*NewMorphTargetAttribute = MoveTemp(*OldMorphTargetAttribute);

	Mesh.Attributes()->AttachMorphTargetAttribute(ActualNewName, NewMorphTargetAttribute);
	Mesh.Attributes()->RemoveMorphTargetAttribute(InOldName);

	Trackers.MorphTargetChangeTracker.HandleRenameMorphTarget(InOldName, ActualNewName);

	return ActualNewName;
}

FName USkeletalMeshBackedDynamicMeshComponent::RenameMorphTarget(FName InOldName, FName InNewName)
{
	FChangeScope ChangeScope(this);
	FName Result = NAME_None;
	EditMesh([&](FDynamicMesh3& Mesh) { Result = RenameMorphTargetDirect(Mesh, InOldName, InNewName); });
	MarkDirtyDirect();
	return Result;
}

void USkeletalMeshBackedDynamicMeshComponent::RemoveMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames)
{
	using namespace UE::Geometry;

	TArray<FName> MorphsToRemove;
	for (const FName& Name : InNames)
	{
		if (GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(Name))
		{
			MorphsToRemove.Add(Name);
		}
	}

	if (MorphsToRemove.IsEmpty())
	{
		return;
	}

	for (const FName& Name : MorphsToRemove)
	{
		Mesh.Attributes()->RemoveMorphTargetAttribute(Name);
	}

	for (const FName& Name : MorphsToRemove)
	{
		Trackers.MorphTargetChangeTracker.HandleRemoveMorphTarget(Name);
	}
}

void USkeletalMeshBackedDynamicMeshComponent::RemoveMorphTargets(const TArray<FName>& InNames)
{
	FChangeScope ChangeScope(this);
	EditMesh([&](FDynamicMesh3& Mesh) { RemoveMorphTargetsDirect(Mesh, InNames); });
	MarkDirtyDirect();
}

TArray<FName> USkeletalMeshBackedDynamicMeshComponent::DuplicateMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames)
{
	using namespace UE::Geometry;

	TArray<FName> MorphsToDuplicate;
	for (const FName& Name : InNames)
	{
		if (GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(Name))
		{
			MorphsToDuplicate.Add(Name);
		}
	}

	if (MorphsToDuplicate.IsEmpty())
	{
		return {};
	}

	TArray<FName> NewMorphs;
	NewMorphs.Reserve(MorphsToDuplicate.Num());
	for (const FName& Name : MorphsToDuplicate)
	{
		const FName DuplicateName = GetAvailableMorphTargetName(Name);
		NewMorphs.Add(DuplicateName);

		FDynamicMeshMorphTargetAttribute* SourceMorphTargetAttribute = Mesh.Attributes()->GetMorphTargetAttribute(Name);

		FDynamicMeshMorphTargetAttribute* DuplicatedMorphTargetAttribute = new FDynamicMeshMorphTargetAttribute(&Mesh);
		DuplicatedMorphTargetAttribute->Copy(*SourceMorphTargetAttribute);
		DuplicatedMorphTargetAttribute->SetName(DuplicateName);

		Mesh.Attributes()->AttachMorphTargetAttribute(DuplicateName, DuplicatedMorphTargetAttribute);

		Trackers.MorphTargetChangeTracker.HandleAddMorphTarget(DuplicateName);
		Trackers.MorphTargetChangeTracker.HandleEditMorphTarget(DuplicateName);
	}

	return NewMorphs;
}

TArray<FName> USkeletalMeshBackedDynamicMeshComponent::DuplicateMorphTargets(const TArray<FName>& InNames)
{
	FChangeScope ChangeScope(this);
	TArray<FName> NewMorphs;
	EditMesh([&](FDynamicMesh3& Mesh) { NewMorphs = DuplicateMorphTargetsDirect(Mesh, InNames); });
	MarkDirtyDirect();
	return NewMorphs;
}

void USkeletalMeshBackedDynamicMeshComponent::MirrorMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames)
{
	using namespace UE::Geometry;
	check(ReadOnlySymmetry);

	TArray<FName> MorphsToEdit;
	for (const FName& Name : InNames)
	{
		if (GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(Name))
		{
			MorphsToEdit.Add(Name);
		}
	}

	if (MorphsToEdit.IsEmpty())
	{
		return;
	}

	for (const FName& Name : MorphsToEdit)
	{
		// nullptr OutDeltaChange: this path's change tracking is the surrounding FChangeScope + MorphTargetChangeTracker.
		SkeletalMeshToolsHelper::MirrorMorphTargetOnMesh(Mesh, Name, *ReadOnlySymmetry);
	}

	for (const FName& Name : MorphsToEdit)
	{
		Trackers.MorphTargetChangeTracker.HandleEditMorphTarget(Name);
	}
}

void USkeletalMeshBackedDynamicMeshComponent::MirrorMorphTargets(const TArray<FName>& InNames)
{
	UpdateSymmetryIfNeeded();
	FChangeScope ChangeScope(this);
	EditMesh([&](FDynamicMesh3& Mesh) { MirrorMorphTargetsDirect(Mesh, InNames); });
	MarkDirtyDirect();
}

void USkeletalMeshBackedDynamicMeshComponent::FlipMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames)
{
	using namespace UE::Geometry;
	check(ReadOnlySymmetry);

	TArray<FName> MorphsToEdit;
	for (const FName& Name : InNames)
	{
		if (GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(Name))
		{
			MorphsToEdit.Add(Name);
		}
	}

	if (MorphsToEdit.IsEmpty())
	{
		return;
	}

	for (const FName& Name : MorphsToEdit)
	{
		FDynamicMeshMorphTargetAttribute* SourceMorphTargetAttribute = Mesh.Attributes()->GetMorphTargetAttribute(Name);

		ParallelFor(Mesh.MaxVertexID(), [&](int32 VertID)
		{
			if (!Mesh.IsVertex(VertID))
			{
				return;
			}

			FVector Position = Mesh.GetVertex(VertID);

			FVector Delta;
			SourceMorphTargetAttribute->GetValue(VertID, Delta);

			constexpr bool bForceSameSizeWithGaps = true;
			TArray<int32> MirroredVertID;
			ReadOnlySymmetry->GetMirrorVertexROI({VertID}, MirroredVertID, bForceSameSizeWithGaps);

			// INDEX_NONE means either an on-plane vert or an unmatched vert with no mirror pair.
			// Constrain (Position + Delta) to the symmetry plane: if the constraint changes the delta,
			// the delta has a symmetry-axis component, indicating the vert is on-plane — flip it.
			// If unchanged, the vert is either unmatched or has no axis component to flip — skip.
			if (MirroredVertID[0] == INDEX_NONE)
			{
				TArray<FVector> ConstrainedPosition;
				ConstrainedPosition.Add(Position + Delta);
				ReadOnlySymmetry->ApplySymmetryPlaneConstraints({VertID}, ConstrainedPosition);

				FVector ConstrainedDelta = ConstrainedPosition[0] - Position;

				if (!ConstrainedDelta.Equals(Delta))
				{
					FVector FlippedDelta = ReadOnlySymmetry->GetMirroredAxis(Delta);
					SourceMorphTargetAttribute->SetValue(VertID, FlippedDelta);
				}
			}
			else
			{
				// Process positive side only; read both deltas, then swap with mirrored axis
				if (Position.X < 0)
				{
					return;
				}

				int32 NegVertID = MirroredVertID[0];

				FVector NegDelta;
				SourceMorphTargetAttribute->GetValue(NegVertID, NegDelta);

				FVector MirroredNegDelta = ReadOnlySymmetry->GetMirroredAxis(NegDelta);
				FVector MirroredPosDelta = ReadOnlySymmetry->GetMirroredAxis(Delta);

				// Swap: pos_vert gets mirrored neg delta, neg_vert gets mirrored pos delta
				SourceMorphTargetAttribute->SetValue(VertID, MirroredNegDelta);
				SourceMorphTargetAttribute->SetValue(NegVertID, MirroredPosDelta);
			}
		});
	}

	for (const FName& Name : MorphsToEdit)
	{
		Trackers.MorphTargetChangeTracker.HandleEditMorphTarget(Name);
	}
}

void USkeletalMeshBackedDynamicMeshComponent::FlipMorphTargets(const TArray<FName>& InNames)
{
	UpdateSymmetryIfNeeded();
	FChangeScope ChangeScope(this);
	EditMesh([&](FDynamicMesh3& Mesh) { FlipMorphTargetsDirect(Mesh, InNames); });
	MarkDirtyDirect();
}

void USkeletalMeshBackedDynamicMeshComponent::GenerateFlippedMorphTargets(const TArray<TPair<FName, FName>>& InPairs)
{
	UpdateSymmetryIfNeeded();

	const TArray<FName>& Existing = GetMorphTargetChangeTracker().GetCurrentMorphTargetNames();

	TArray<TPair<FName, FName>> PairsToEdit;
	PairsToEdit.Reserve(InPairs.Num());
	for (const TPair<FName, FName>& Pair : InPairs)
	{
		if (Existing.Contains(Pair.Key))
		{
			PairsToEdit.Add(Pair);
		}
	}

	if (PairsToEdit.IsEmpty())
	{
		return;
	}

	FChangeScope ChangeScope(this);

	EditMesh([&](FDynamicMesh3& Mesh)
	{
		using namespace UE::Geometry;

		// 1. Add any missing destinations and full-copy source deltas into each destination.
		TArray<FName> DestNames;
		DestNames.Reserve(PairsToEdit.Num());
		for (const TPair<FName, FName>& Pair : PairsToEdit)
		{
			if (!GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(Pair.Value))
			{
				AddMorphTargetDirect(Mesh, Pair.Value);
			}

			FDynamicMeshMorphTargetAttribute* SourceAttr = Mesh.Attributes()->GetMorphTargetAttribute(Pair.Key);
			FDynamicMeshMorphTargetAttribute* DestAttr = Mesh.Attributes()->GetMorphTargetAttribute(Pair.Value);
			DestAttr->Copy(*SourceAttr);

			DestNames.Add(Pair.Value);
		}

		// 2. Flip the destinations in place.
		FlipMorphTargetsDirect(Mesh, DestNames);
	});

	MarkDirtyDirect();
}

FName USkeletalMeshBackedDynamicMeshComponent::MergeMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames)
{
	using namespace UE::Geometry;

	// Validate: keep only names that actually exist on the current mesh.
	TArray<FName> MorphsToMerge;
	for (const FName& Name : InNames)
	{
		if (GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(Name))
		{
			MorphsToMerge.Add(Name);
		}
	}
	if (MorphsToMerge.Num() < 2)
	{
		return NAME_None;
	}

	// Name the merged morph after the first source with a "_Merged" suffix.
	const FString MergedBaseName = MorphsToMerge[0].ToString() + TEXT("_Merged");
	const FName MergedName = GetAvailableMorphTargetName(FName(*MergedBaseName));

	// Scope the raw attribute pointers so they cannot leak past the RemoveMorphTargetsDirect call below.
	{
		TArray<FDynamicMeshMorphTargetAttribute*> SourceAttrs;
		SourceAttrs.Reserve(MorphsToMerge.Num());
		for (const FName& Name : MorphsToMerge)
		{
			SourceAttrs.Add(Mesh.Attributes()->GetMorphTargetAttribute(Name));
		}

		FDynamicMeshMorphTargetAttribute* MergedAttr = new FDynamicMeshMorphTargetAttribute(&Mesh);
		MergedAttr->SetName(MergedName);
		Mesh.Attributes()->AttachMorphTargetAttribute(MergedName, MergedAttr);

		ParallelFor(Mesh.MaxVertexID(), [&](int32 VertID)
		{
			if (!Mesh.IsVertex(VertID))
			{
				return;
			}

			FVector Sum = FVector::ZeroVector;
			for (FDynamicMeshMorphTargetAttribute* Src : SourceAttrs)
			{
				FVector Delta;
				Src->GetValue(VertID, Delta);
				Sum += Delta;
			}
			MergedAttr->SetValue(VertID, Sum);
		});
	}

	Trackers.MorphTargetChangeTracker.HandleAddMorphTarget(MergedName);
	Trackers.MorphTargetChangeTracker.HandleEditMorphTarget(MergedName);

	// Consume the sources: their deltas have been folded into the merged attribute, so the originals are no longer needed.
	RemoveMorphTargetsDirect(Mesh, MorphsToMerge);

	return MergedName;
}

FName USkeletalMeshBackedDynamicMeshComponent::MergeMorphTargets(const TArray<FName>& InNames)
{
	FChangeScope ChangeScope(this);
	FName Result = NAME_None;
	EditMesh([&](FDynamicMesh3& Mesh) { Result = MergeMorphTargetsDirect(Mesh, InNames); });
	MarkDirtyDirect();
	return Result;
}

void USkeletalMeshBackedDynamicMeshComponent::ApplyWeightToMorphTargetDirect(FDynamicMesh3& Mesh, FName InName, float InWeight)
{
	using namespace UE::Geometry;

	if (!GetMorphTargetChangeTracker().GetCurrentMorphTargetNames().Contains(InName))
	{
		return;
	}
	if (FMath::IsNearlyEqual(InWeight, 1.0f))
	{
		return;
	}

	FDynamicMeshMorphTargetAttribute* Attr = Mesh.Attributes()->GetMorphTargetAttribute(InName);
	if (!Attr)
	{
		return;
	}

	ParallelFor(Mesh.MaxVertexID(), [&](int32 VertID)
	{
		if (!Mesh.IsVertex(VertID))
		{
			return;
		}

		FVector Delta;
		Attr->GetValue(VertID, Delta);
		Attr->SetValue(VertID, Delta * InWeight);
	});

	Trackers.MorphTargetChangeTracker.HandleEditMorphTarget(InName);
}

void USkeletalMeshBackedDynamicMeshComponent::ApplyWeightToMorphTarget(FName InName, float InWeight)
{
	FChangeScope ChangeScope(this);
	EditMesh([&](FDynamicMesh3& Mesh) { ApplyWeightToMorphTargetDirect(Mesh, InName, InWeight); });
	MarkDirtyDirect();
}

void USkeletalMeshBackedDynamicMeshComponent::MarkMorphTargetEdited(FName InName)
{
	FTrackerChangeScope Scope(this);
	
	// Mesh should be marked dirty during tool commit already 
	Trackers.MorphTargetChangeTracker.HandleEditMorphTarget(InName);

}

bool USkeletalMeshBackedDynamicMeshComponent::IsSkeletonDirty() const
{
	return GetSkeletonChangeTracker().GetChangeCount() != 0;
}

bool USkeletalMeshBackedDynamicMeshComponent::IsDirty() const
{
	return GetChangeCount() != 0;
}

int32 USkeletalMeshBackedDynamicMeshComponent::GetChangeCount() const
{
	return Trackers.ChangeCount;
}

USkeletalMesh* USkeletalMeshBackedDynamicMeshComponent::GetSkeletalMesh() const
{
	return WeakSkeletalMesh.Get();
}

void USkeletalMeshBackedDynamicMeshComponent::FlushPendingEvents()
{
	if (bPendingChangeEvent)
	{
		bPendingChangeEvent = false;
		OnChangedEvent.Broadcast();
	}
	
	if (bPendingSkeletonChangeEvent)
	{
		bPendingSkeletonChangeEvent = false;
		OnSkeletonChangedEvent.Broadcast();
	}
}

void USkeletalMeshBackedDynamicMeshComponent::BeginDestroy()
{
	if (TransactionStateChangedHandle.IsValid())
	{
		if (GEditor && GEditor->Trans)
		{
			if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
			{
				TransBuffer->OnTransactionStateChanged().Remove(TransactionStateChangedHandle);
			}
		}
		TransactionStateChangedHandle.Reset();
	}

	Super::BeginDestroy();
}

void USkeletalMeshBackedDynamicMeshComponent::OnTransactionStateChanged(
	const FTransactionContext& /*Context*/, ETransactionStateEventType EventType)
{
	switch (EventType)
	{
	case ETransactionStateEventType::TransactionFinalized:
	case ETransactionStateEventType::UndoRedoFinalized:
		FlushPendingEvents();
		break;

	// On cancel, the transaction record is dropped without invoking Apply/Revert, but any live
	// mutations made inside the scope are still in our Trackers — they are NOT auto-rolled back.
	// We broadcast so observers reflect the actual post-mutation state. Callers should avoid
	// mutating this component until they're certain the transaction can close; cancelling
	// after mutating leaves the component in the post-mutation state with no built-in rollback.
	case ETransactionStateEventType::TransactionCanceled:
		FlushPendingEvents();
		break;

	default:
		break;
	}
}

void USkeletalMeshBackedDynamicMeshComponent::ResetTrackersDirect()
{
	Trackers.InitialAssetMesh = GetMeshCopy();

	Trackers.ChangeCount = 0;
	Trackers.BaseGeometryChangeCount = 0;
	
	Trackers.RefSkeleton = WeakSkeletalMesh->GetRefSkeleton();
	Trackers.RefSkeleton.GetBoneAbsoluteTransforms(Trackers.ComponentSpaceBoneTransformsRefPose);
	Trackers.SkeletonChangeTracker.Init(Trackers.RefSkeleton.GetRawBoneNum());
	
	TArray<FName> MorphTargetNames;
	GetMesh()->Attributes()->GetMorphTargetAttributes().GenerateKeyArray(MorphTargetNames);
	Trackers.MorphTargetChangeTracker.Init(MorphTargetNames);
}

FName USkeletalMeshBackedDynamicMeshComponent::GetAvailableMorphTargetName(FName InNewName, FName InOldName) const
{
	TArray<FName> MorphTargets = GetMorphTargetChangeTracker().GetCurrentMorphTargetNames();

	MorphTargets.Remove(InOldName);
	
	FName ActualName = InNewName;	
	while (MorphTargets.Contains(ActualName))
	{
		ActualName.SetNumber(ActualName.GetNumber() + 1);
	}

	return ActualName;
}

EMeshLODIdentifier USkeletalMeshBackedDynamicMeshComponent::GetValidEditingLOD(EMeshLODIdentifier InLOD) const
{
	constexpr bool bSkipAutoGenerated= true;
	TArray<EMeshLODIdentifier> AssetAvailableLODs = UE::ToolTarget::GetAvailableLODs(WeakSkeletalMesh.Get(), bSkipAutoGenerated);

	if (!AssetAvailableLODs.Contains(InLOD))
	{
		return AssetAvailableLODs[0];
	}

	return InLOD;
}

TSharedPtr<FDynamicMesh3> USkeletalMeshBackedDynamicMeshComponent::GetMeshCopy()
{
	TSharedPtr<FDynamicMesh3> Mesh = MakeShared<FDynamicMesh3>();
	ProcessMesh([&](const FDynamicMesh3& ReadMesh) { *Mesh = ReadMesh; });
	return Mesh;	
}

void USkeletalMeshBackedDynamicMeshComponent::MarkDirtyDirect()
{
	Trackers.ChangeCount++;
}

void USkeletalMeshBackedDynamicMeshComponent::MarkBaseGeometryDirtyDirect()
{
	Trackers.BaseGeometryChangeCount++;
}

void USkeletalMeshBackedDynamicMeshComponent::MarkDirty()
{
	FTrackerChangeScope ChangeScope(this);

	MarkDirtyDirect();
}

void USkeletalMeshBackedDynamicMeshComponent::MarkBaseGeometryDirty()
{
	FTrackerChangeScope ChangeScope(this);

	MarkBaseGeometryDirtyDirect();	
}


USkeletalMeshBackedDynamicMeshComponent::FTrackerChange::FTrackerChange(USkeletalMeshBackedDynamicMeshComponent* InComponent)
{
	OldTrackers = InComponent->Trackers;
	NewTrackers = OldTrackers; 
}

FString USkeletalMeshBackedDynamicMeshComponent::FTrackerChange::ToString() const
{
	return TEXT("State Changed");
}

void USkeletalMeshBackedDynamicMeshComponent::FTrackerChange::Apply(UObject* Object)
{
	if (USkeletalMeshBackedDynamicMeshComponent* Component = Cast<USkeletalMeshBackedDynamicMeshComponent>(Object))
	{
		if (bIsCommitChange)
		{
			Component->bExpectAssetChange = true;
		}
		if (ShouldEnqueueMeshChangedEvent())
		{
			Component->bPendingChangeEvent = true;
		}
		if (ShouldEnqueueSkeletonChangedEvent())
		{
			Component->bPendingSkeletonChangeEvent = true;
		}
		Component->Trackers = NewTrackers;
	}	
}

void USkeletalMeshBackedDynamicMeshComponent::FTrackerChange::Revert(UObject* Object)
{
	if (USkeletalMeshBackedDynamicMeshComponent* Component = Cast<USkeletalMeshBackedDynamicMeshComponent>(Object))
	{
		if (bIsCommitChange)
		{
			Component->bExpectAssetChange = true;
		}
		if (ShouldEnqueueMeshChangedEvent())
		{
			Component->bPendingChangeEvent = true;
		}
		if (ShouldEnqueueSkeletonChangedEvent())
		{
			Component->bPendingSkeletonChangeEvent = true;
		}
		Component->Trackers = OldTrackers;
	}	
}

void USkeletalMeshBackedDynamicMeshComponent::FChangeScope::MarkAsCommitChange()
{
	if (TrackerChange)
	{
		TrackerChange->bIsCommitChange = true;
	}
}

void USkeletalMeshBackedDynamicMeshComponent::FTrackerChange::Close(USkeletalMeshBackedDynamicMeshComponent* InComponent)
{
	NewTrackers = InComponent->Trackers;	
}

bool USkeletalMeshBackedDynamicMeshComponent::FTrackerChange::ShouldEnqueueMeshChangedEvent() const
{
	return NewTrackers.ChangeCount != OldTrackers.ChangeCount;
}

bool USkeletalMeshBackedDynamicMeshComponent::FTrackerChange::ShouldEnqueueSkeletonChangedEvent() const
{
	return NewTrackers.SkeletonChangeTracker.GetChangeCount() != OldTrackers.SkeletonChangeTracker.GetChangeCount();
}

USkeletalMeshBackedDynamicMeshComponent::FChangeScope::FChangeScope(USkeletalMeshBackedDynamicMeshComponent* InComponent, bool bInRecordMeshChange)
{
	Component = InComponent;
	bRecordMeshChange = bInRecordMeshChange;
	
	if (GUndo)
	{
		TrackerChange = MakeUnique<FTrackerChange>(Component);

		if (bRecordMeshChange)
		{
			OldMesh = Component->GetMeshCopy();
			NewMesh = OldMesh;
		}
	}
	
}

USkeletalMeshBackedDynamicMeshComponent::FChangeScope::~FChangeScope()
{
	if (GUndo)
	{
		{
			TrackerChange->Close(Component);
			if (TrackerChange->ShouldEnqueueMeshChangedEvent())
			{
				Component->bPendingChangeEvent = true;
			}
			if (TrackerChange->ShouldEnqueueSkeletonChangedEvent())
			{
				Component->bPendingSkeletonChangeEvent = true;
			}
			GUndo->StoreUndo(Component, MoveTemp(TrackerChange));	
		}
		
		if (bRecordMeshChange)
		{
			NewMesh = Component->GetMeshCopy();

			TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(OldMesh, NewMesh);

			Component->GetDynamicMesh()->Modify();
			GUndo->StoreUndo(Component->GetDynamicMesh(), MoveTemp(ReplaceChange));	
		}
		

	}
}

const UE::Geometry::FGroupTopology& USkeletalMeshBackedDynamicMeshComponent::GetGroupTopology()
{
	UpdateGroupTopologyIfNeeded();
	check(CachedGroupTopology);
	return *CachedGroupTopology;
}

void USkeletalMeshBackedDynamicMeshComponent::UpdateGroupTopologyIfNeeded()
{
	if (BaseGeometryVersionForCurrentGroupTopology == Trackers.BaseGeometryChangeCount)
	{
		return;
	}

	BaseGeometryVersionForCurrentGroupTopology = Trackers.BaseGeometryChangeCount;

	GetDynamicMesh()->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		CachedGroupTopology = MakePimpl<UE::Geometry::FGroupTopology>(&Mesh, true);
	});
}

void USkeletalMeshBackedDynamicMeshComponent::UpdateSymmetryIfNeeded()
{
	using namespace UE::Geometry;

	if (BaseGeometryVersionForCurrentSymmetry == Trackers.BaseGeometryChangeCount)
	{
		return;
	}

	BaseGeometryVersionForCurrentSymmetry = Trackers.BaseGeometryChangeCount;
	
	GetDynamicMesh()->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		FAxisAlignedBox3d Bounds = Mesh.GetBounds(true);
		TArray<FVector3d> PreferSymmetryPlaneNormal;
		PreferSymmetryPlaneNormal.Add(FVector3d::UnitX());

		FFrame3d SymmetryFrame(FVector3d::Zero(), FVector3d::UnitX());		

		constexpr bool bForceComputeVertexPairs = true;	
			
		ReadOnlySymmetry = MakePimpl<FMeshPlanarSymmetry>();
		ReadOnlySymmetry->Initialize(const_cast<FDynamicMesh3*>(&Mesh), Bounds, SymmetryFrame, bForceComputeVertexPairs);
	});
}

const UE::Geometry::FMeshPlanarSymmetry* USkeletalMeshBackedDynamicMeshComponent::GetBaseMeshSymmetry()
{
	UpdateSymmetryIfNeeded();
	return ReadOnlySymmetry.Get();
}

#undef LOCTEXT_NAMESPACE
