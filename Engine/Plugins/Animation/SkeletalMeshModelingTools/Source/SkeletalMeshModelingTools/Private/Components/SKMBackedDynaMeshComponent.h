// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/DynamicMeshComponent.h"
#include "ReferenceSkeleton.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include "SKMBackedDynaMeshComponent.generated.h"

#define UE_API SKELETALMESHMODELINGTOOLS_API


namespace UE::Geometry
{
	class FMeshPlanarSymmetry;
	struct FDynamicSubmesh3;
	class FGroupTopology;
}

class USkeletonModifier;

struct FTransactionContext;
enum class ETransactionStateEventType : uint8;


UCLASS(MinimalAPI, Transient)
class USkeletalMeshBackedDynamicMeshComponent :
	public UDynamicMeshComponent
{
	GENERATED_BODY()
public:	
	
	struct FSkeletonChangeTracker
	{
		void Init(int32 NumBones);
		void HandleSkeletonChanged(const TArray<int32> ToolBoneIndexTracker);
		int32 GetChangeCount() const;
		const TArray<int32>& GetBoneIndexTracker() const;

	private:
		// Used to update retarget settings
		TArray<int32> BoneIndexTracker;

		int32 ChangeCount = 0;
	};

	struct FMorphTargetChangeTracker
	{
		void Init(const TArray<FName> ExistingMorphTargets);
		void HandleRenameMorphTarget(FName CurrentName, FName NewName);
		void HandleRemoveMorphTarget(FName Name);
		void HandleAddMorphTarget(FName Name);
		void HandleEditMorphTarget(FName Name);
		FName GetCurrentMorphTargetName(FName OriginalName) const;
		FName GetOriginalMorphTargetName(FName CurrentName) const;
		const TSet<FName>& GetEditedMorphTargets() const;
		struct FNameInfo
		{
			FName OldName;
			FName NewName;
		};
		TArray<FNameInfo> GetCurvesToRename() const;
		TArray<FName> GetCurvesToRemove() const;
		TArray<FName> GetCurvesToAdd() const;

		TArray<FName> GetCurrentMorphTargetNames() const;
	
	private:
		TMap<FName, FName> OriginalNameToCurrentName;
		TMap<FName, FName> CurrentNameToOriginalName;
		TSet<FName> EditedMorphTargets;

	};
	

	EMeshLODIdentifier Init(USkeletalMesh* InSkeletalMesh, EMeshLODIdentifier InLOD);
	EMeshLODIdentifier GetLOD() const;
	bool CommitToSkeletalMesh();

	void DiscardChanges();

	void MarkDirty();
	void MarkBaseGeometryDirty();
	void HandleSkeletonChange(USkeletonModifier* InModifier);
	void ForwardVisibilityChangeRequest(bool bInVisible);

	void HandleGeometryUpdate(
		const FName EditingMorphTargetName,
		const FDynamicMesh3& PosedMesh,
		const TArray<FMatrix>& BoneMatrices,
		const TMap<FName, float>& MorphTargetWeights,
		const UE::Geometry::FDynamicSubmesh3* InSubmesh = nullptr
		);

	FName AddMorphTarget(FName InName);
	TArray<FName> AddMorphTargetsIfMissing(const TArray<FName>& Names);
	FName RenameMorphTarget(FName InOldName, FName InNewName);
	void RemoveMorphTargets(const TArray<FName>& InNames);
	TArray<FName> DuplicateMorphTargets(const TArray<FName>& InNames);
	void MirrorMorphTargets(const TArray<FName>& InNames);
	void FlipMorphTargets(const TArray<FName>& InNames);
	FName MergeMorphTargets(const TArray<FName>& InNames);
	void  ApplyWeightToMorphTarget(FName InName, float InWeight);
	void GenerateFlippedMorphTargets(const TArray<TPair<FName, FName>>& InPairs);

	void MarkMorphTargetEdited(FName InName);
	
	const FSkeletonChangeTracker& GetSkeletonChangeTracker() const {return Trackers.SkeletonChangeTracker;}
	const FMorphTargetChangeTracker& GetMorphTargetChangeTracker() const {return Trackers.MorphTargetChangeTracker;}
	const FReferenceSkeleton& GetRefSkeleton() const {return Trackers.RefSkeleton;}
	const TArray<FTransform>& GetComponentSpaceBoneTransformsRefPose() const {return Trackers.ComponentSpaceBoneTransformsRefPose;}

	bool IsSkeletonDirty() const;
	bool IsDirty() const;
	int32 GetChangeCount() const;
	

	USkeletalMesh* GetSkeletalMesh() const;

	const UE::Geometry::FGroupTopology& GetGroupTopology();

	// Lazily rebuilds when base geometry changes; pointer is invalidated by such edits.
	const UE::Geometry::FMeshPlanarSymmetry* GetBaseMeshSymmetry();
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRequestingVisibilityChange, bool);
	FOnRequestingVisibilityChange& GetOnRequestingVisibilityChange() {return OnRequestingVisibilityChangeDelegate;}

	// Set before expected asset changes (commit, undo/redo of commit).
	// Checked for re-entrancy in CommitToSkeletalMesh.
	// Checked and cleared by HandleSkeletalMeshChanged to distinguish our own changes from external ones.
	bool IsExpectingAssetChange() const { return bExpectAssetChange; }
	void ClearExpectAssetChange() { bExpectAssetChange = false; }

	void FlushPendingEvents();
	FSimpleMulticastDelegate& OnChanged() {return OnChangedEvent; };
	FSimpleMulticastDelegate& OnSkeletonChanged() {return OnSkeletonChangedEvent; };

	virtual void BeginDestroy() override;
protected:
	void ResetTrackersDirect();
	FName GetAvailableMorphTargetName(FName InNewName, FName InOldName = NAME_None) const;
	EMeshLODIdentifier GetValidEditingLOD(EMeshLODIdentifier InLOD) const;

	// Direct mesh-mutation primitives. Caller is responsible for opening FChangeScope, running EditMesh
	// once for the whole batch, and calling MarkDirtyDirect afterwards. Trackers are updated here so
	// subsequent Direct calls in the same batch see the new state.
	FName AddMorphTargetDirect(FDynamicMesh3& Mesh, FName InName);
	FName RenameMorphTargetDirect(FDynamicMesh3& Mesh, FName InOldName, FName InNewName);
	void RemoveMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames);
	TArray<FName> DuplicateMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames);
	void MirrorMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames);
	void FlipMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames);
	FName MergeMorphTargetsDirect(FDynamicMesh3& Mesh, const TArray<FName>& InNames);
	void ApplyWeightToMorphTargetDirect(FDynamicMesh3& Mesh, FName InName, float InWeight);

	TSharedPtr<FDynamicMesh3> GetMeshCopy();
	void MarkDirtyDirect();
	void MarkBaseGeometryDirtyDirect();

	struct FTrackers
	{
		TSharedPtr<FDynamicMesh3> InitialAssetMesh;
		
		int32 ChangeCount = 0;
		// Base Geometry determines if symmetry tracking needs to be updated
		int32 BaseGeometryChangeCount = 0;	
		
		FReferenceSkeleton RefSkeleton;
		TArray<FTransform> ComponentSpaceBoneTransformsRefPose;
		FSkeletonChangeTracker SkeletonChangeTracker;
		FMorphTargetChangeTracker MorphTargetChangeTracker;
	};

	FTrackers Trackers;

	TWeakObjectPtr<USkeletalMesh> WeakSkeletalMesh;
	EMeshLODIdentifier LOD = EMeshLODIdentifier::LOD0;
	int32 LODIndex = 0;

	bool bExpectAssetChange = false;
	
	bool bPendingChangeEvent = false;
	FSimpleMulticastDelegate OnChangedEvent;
	bool bPendingSkeletonChangeEvent = false;
	FSimpleMulticastDelegate OnSkeletonChangedEvent;

	FOnRequestingVisibilityChange OnRequestingVisibilityChangeDelegate;

	// Fires pending change events when the containing transaction settles (forward commit,
	// undo/redo replay, or cancel). Replaces the previous Tick-driven flush so observers
	// see a coherent post-transaction state before subsequent code reads it.
	void OnTransactionStateChanged(const FTransactionContext& Context, ETransactionStateEventType EventType);
	FDelegateHandle TransactionStateChangedHandle;

	class FTrackerChange : public FToolCommandChange
	{
	public:
		FTrackers OldTrackers;
		FTrackers NewTrackers;
		bool bIsCommitChange = false;
		FTrackerChange(USkeletalMeshBackedDynamicMeshComponent* InComponent);
		virtual FString ToString() const override;
		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;
		void Close(USkeletalMeshBackedDynamicMeshComponent* InComponent);	
		
		bool ShouldEnqueueMeshChangedEvent() const;
		bool ShouldEnqueueSkeletonChangedEvent() const;
	};
	
	// Records into current Transaction if there is one
	class FChangeScope 
	{
	public:
		FChangeScope(USkeletalMeshBackedDynamicMeshComponent* InComponent, bool bInRecordMeshChange = true);
		~FChangeScope();

		void MarkAsCommitChange();

	private:
		TUniquePtr<FTrackerChange> TrackerChange;
		TSharedPtr<FDynamicMesh3> OldMesh;
		TSharedPtr<FDynamicMesh3> NewMesh;

		USkeletalMeshBackedDynamicMeshComponent* Component = nullptr;
		bool bRecordMeshChange = false;

		friend class FTrackerChange;
	};

	class FTrackerChangeScope : public FChangeScope
	{
	public:
		FTrackerChangeScope(USkeletalMeshBackedDynamicMeshComponent* InComponent) : FChangeScope(InComponent, false) {}
	};


	
	friend class FChangeScope;
	friend class FTrackerChange;

	// Query only symmetry, not allow to modify mesh
	TPimplPtr<UE::Geometry::FMeshPlanarSymmetry> ReadOnlySymmetry;
	int32 BaseGeometryVersionForCurrentSymmetry = -1;
	void UpdateSymmetryIfNeeded();

	// Cached group topology, lazily built and invalidated when base geometry changes
	TPimplPtr<UE::Geometry::FGroupTopology> CachedGroupTopology;
	int32 BaseGeometryVersionForCurrentGroupTopology = -1;
	void UpdateGroupTopologyIfNeeded();
};

#undef UE_API
