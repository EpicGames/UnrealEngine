// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKMMTMirroringPropertiesAxisCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "Widgets/SSKMEnumPropertyComboBox.h"

namespace UE::SkeletalMeshModelingTools
{
	TSharedRef<IPropertyTypeCustomization> FSKMMTMirroringPropertiesAxisCustomization::MakeInstance()
	{
		return MakeShared<FSKMMTMirroringPropertiesAxisCustomization>();
	}

	void FSKMMTMirroringPropertiesAxisCustomization::CustomizeHeader(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
	}

	void FSKMMTMirroringPropertiesAxisCustomization::CustomizeChildren(
		TSharedRef<IPropertyHandle> InStructPropertyHandle, 
		IDetailChildrenBuilder& StructBuilder, 
		IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		TArray<UObject*> OuterObjects;
		InStructPropertyHandle->GetOuterObjects(OuterObjects);

		const bool bValid = OuterObjects.Num() == 1 && OuterObjects[0] && OuterObjects[0]->GetClass() == UMirroringProperties::StaticClass();
		if (!ensureMsgf(bValid, TEXT("Unexpected unexpected objects wheen trying to customize the Mirroring Properties' Mirroring Axis")))
		{
			return;
		}
		const TWeakObjectPtr<UMirroringProperties> WeakMirroringProperties = CastChecked<UMirroringProperties>(OuterObjects[0]);

		const TAttribute<EVisibility> VisibilityAttribute = TAttribute<EVisibility>::CreateLambda([WeakMirroringProperties]()
			{
				const bool bVisible =
					WeakMirroringProperties.IsValid() &&
					WeakMirroringProperties->ParentTool.IsValid() &&
					WeakMirroringProperties->ParentTool->GetOperation() != EEditingOperation::Create;

				return bVisible ? EVisibility::Visible : EVisibility::Hidden;
			});

		const TAttribute<bool> EnabledAttribute = TAttribute<bool>::CreateLambda([WeakMirroringProperties]()
			{
				return
					WeakMirroringProperties.IsValid() &&
					WeakMirroringProperties->ParentTool.IsValid() &&
					!WeakMirroringProperties->ParentTool->GetSelection().IsEmpty() &&
					WeakMirroringProperties->ParentTool->GetOperation() != EEditingOperation::Parent;
			});

		uint32 NumChildren = 0;
		InStructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			if (const TSharedPtr<IPropertyHandle> ChildProperty = InStructPropertyHandle->GetChildHandle(ChildIndex))
			{
				const FProperty* Property = ChildProperty->GetProperty();
				const FName PropertyName = Property ? Property->GetFName() : NAME_None;

				if (PropertyName == GET_MEMBER_NAME_CHECKED(FMirrorOptions, MirrorAxis))
				{	
					StructBuilder.AddCustomRow(ChildProperty->GetPropertyDisplayName())
						.NameContent()
						[
							ChildProperty->CreatePropertyNameWidget()
						]
						.ValueContent()
						[
							SNew(SSKMEnumPropertyComboBox<EAxis::Type>, ChildProperty.ToSharedRef())
							.ExcludedEnumerators({ EAxis::None })
							.Visibility(VisibilityAttribute)
							.IsEnabled(EnabledAttribute)
						]
						.ExtensionContent()
						[
							ChildProperty->CreateDefaultPropertyButtonWidgets()
						];
				}
				else
				{
					StructBuilder.AddProperty(ChildProperty.ToSharedRef());
				}
			}
		}
	}
}
