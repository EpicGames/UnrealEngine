// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

namespace UE::SkeletalMeshModelingTools
{
	/** Property type customization for EAxis in the Skeletal Mesh editor */
	class FSKMMTMirroringPropertiesAxisCustomization
		: public IPropertyTypeCustomization
	{
	public:
		/** Creates an instance of this property type customization */
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	private:
		//~ Begin IPropertyTypeCustomization interface
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization interface
	};
}
