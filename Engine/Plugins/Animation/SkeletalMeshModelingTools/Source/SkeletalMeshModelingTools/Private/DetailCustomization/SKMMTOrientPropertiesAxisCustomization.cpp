// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKMMTOrientPropertiesAxisCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "Widgets/SSKMEnumPropertyComboBox.h"

namespace UE::SkeletalMeshModelingTools
{
	TSharedRef<IPropertyTypeCustomization> FSKMMTOrientPropertiesAxisCustomization::MakeInstance()
	{
		return MakeShared<FSKMMTOrientPropertiesAxisCustomization>();
	}

	void FSKMMTOrientPropertiesAxisCustomization::CustomizeHeader(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
	}

	void FSKMMTOrientPropertiesAxisCustomization::CustomizeChildren(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		IDetailChildrenBuilder& StructBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		TArray<UObject*> OuterObjects;
		InStructPropertyHandle->GetOuterObjects(OuterObjects);

		const bool bValid = OuterObjects.Num() == 1 && OuterObjects[0] && OuterObjects[0]->GetClass() == UOrientingProperties::StaticClass();
		if (!ensureMsgf(bValid, TEXT("Unexpected unexpected objects wheen trying to customize the Orient Properties' Orient Axis")))
		{
			return;
		}
		const TWeakObjectPtr<UOrientingProperties> WeakOrientProperties = CastChecked<UOrientingProperties>(OuterObjects[0]);

		const TAttribute<EVisibility> VisibilityAttribute = TAttribute<EVisibility>::CreateLambda([WeakOrientProperties]()
			{
				const bool bVisible =
					WeakOrientProperties.IsValid() &&
					WeakOrientProperties->ParentTool.IsValid() &&
					WeakOrientProperties->ParentTool->GetOperation() != EEditingOperation::Create;

				return bVisible ? EVisibility::Visible : EVisibility::Hidden;
			});

		const TAttribute<bool> EnabledAttribute = TAttribute<bool>::CreateLambda([WeakOrientProperties]()
			{
				return
					WeakOrientProperties.IsValid() &&
					WeakOrientProperties->ParentTool.IsValid() &&
					!WeakOrientProperties->ParentTool->GetSelection().IsEmpty() &&
					WeakOrientProperties->ParentTool->GetOperation() != EEditingOperation::Parent;
			});

		uint32 NumChildren = 0;
		InStructPropertyHandle->GetNumChildren(NumChildren);
		
		const TSharedPtr<IPropertyHandle> PrimaryOrientAxisProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOrientOptions, Primary));
		const TSharedPtr<IPropertyHandle> SecondaryOrientAxisProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOrientOptions, Secondary));

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			if (const TSharedPtr<IPropertyHandle> ChildProperty = InStructPropertyHandle->GetChildHandle(ChildIndex))
			{
				const FProperty* Property = ChildProperty->GetProperty();
				const FName PropertyName = Property ? Property->GetFName() : NAME_None;

				if (PropertyName == GET_MEMBER_NAME_CHECKED(FOrientOptions, Primary) ||
					PropertyName == GET_MEMBER_NAME_CHECKED(FOrientOptions, Secondary))
				{
					StructBuilder.AddCustomRow(ChildProperty->GetPropertyDisplayName())
						.NameContent()
						[
							ChildProperty->CreatePropertyNameWidget()
						]
						.ValueContent()
						[
							SNew(SSKMEnumPropertyComboBox<EOrientAxis>, ChildProperty.ToSharedRef())
							.ExcludedEnumerators({ EOrientAxis::None })
							.Visibility(VisibilityAttribute)
							.IsEnabled(EnabledAttribute)
						]
						.ExtensionContent()
						[
							ChildProperty->CreateDefaultPropertyButtonWidgets()
						];
				}
				else
				{
					StructBuilder.AddProperty(ChildProperty.ToSharedRef());
				}
			}
		}
	}
}
