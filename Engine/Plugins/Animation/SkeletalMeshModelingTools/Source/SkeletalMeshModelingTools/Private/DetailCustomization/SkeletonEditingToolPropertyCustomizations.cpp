// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonEditingToolPropertyCustomizations.h"

#include "DetailLayoutBuilder.h"
#include "Templates/UniquePtr.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "SKMMTMirroringPropertiesAxisCustomization.h"
#include "SKMMTOrientPropertiesAxisCustomization.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "HAL/PlatformApplicationMisc.h"
#include "InteractiveToolManager.h"
#include "ModelingToolsEditorModeStyle.h"
#include "Selection/PolygonSelectionMechanic.h"


#define LOCTEXT_NAMESPACE "SkeletonEditingToolCustomization"

TSharedRef<IDetailCustomization> FSkeletonEditingPropertiesDetailCustomization::MakeInstance()
{
	return MakeShareable(new FSkeletonEditingPropertiesDetailCustomization);
}

TAttribute<EVisibility> ISkeletonEditingPropertiesDetailCustomization::GetCreationVisibility() const
{
	return TAttribute<EVisibility>::CreateLambda([this]()
	{
		if (!Tool.IsValid())
		{
			return EVisibility::Hidden;
		}
		return Tool->GetOperation() == EEditingOperation::Create ? EVisibility::Visible : EVisibility::Hidden;
	});
}

TAttribute<EVisibility> ISkeletonEditingPropertiesDetailCustomization::GetEditionVisibility() const
{
	return TAttribute<EVisibility>::CreateLambda([this]()
	{
		if (!Tool.IsValid())
		{
			return EVisibility::Hidden;
		}
		return Tool->GetOperation() == EEditingOperation::Create ? EVisibility::Hidden : EVisibility::Visible;
	});
}

TAttribute<bool> ISkeletonEditingPropertiesDetailCustomization::IsEnabledBySelection() const
{
	return TAttribute<bool>::CreateLambda([this]()
	{
		if (!Tool.IsValid())
		{
			return false;
		}
		return !Tool->GetSelection().IsEmpty() && Tool->GetOperation() != EEditingOperation::Parent;
	});
}

void ISkeletonEditingPropertiesDetailCustomization::UpdateProperties(
	IDetailLayoutBuilder& DetailBuilder,
	const TAttribute<EVisibility>& InVisibility,
	const TAttribute<bool>& InEnabled) const
{
	for (const FName& PropName: GetProperties())
	{
		const TSharedRef<IPropertyHandle> Handle = DetailBuilder.GetProperty(PropName);
		if (IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(Handle))
		{
			Row->Visibility(InVisibility);
			if (InEnabled.IsSet())
			{
				Row->IsEnabled(InEnabled);
			}
		}
	}
}

void FSkeletonEditingPropertiesDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TWeakObjectPtr<USkeletonEditingProperties> Properties = GetCustomizedObject<USkeletonEditingProperties>(DetailBuilder);
	if (!GetParentTool(Properties).IsValid())
	{
		return;
	}
	
	RelativeArray.Init(true, 4);
	
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Details"));
	const TSharedRef<IPropertyHandle> NodePropHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkeletonEditingProperties, Name));
	if (NodePropHandle->IsValidHandle())
	{
		TSharedRef<SWidget> ValueWidget = NodePropHandle->CreatePropertyValueWidget();
		ValueWidget->SetEnabled(TAttribute<bool>::Create([this]{ return !Tool->GetSelection().IsEmpty(); }));
		static const FText MultiValues = LOCTEXT("MultipleValues", "Multiple Values");
		
		DetailBuilder.AddPropertyToCategory(NodePropHandle)
			.CustomWidget()
			.NameContent()
			[
				NodePropHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SEditableTextBox)
					.Font(FAppStyle::GetFontStyle( TEXT( "PropertyWindow.NormalFont" ) ))
					.SelectAllTextWhenFocused(true)
					.ClearKeyboardFocusOnCommit(false)
					.SelectAllTextOnCommit(true)
					.Text_Lambda([this]()
					{
						const TArray<FName>& Bones = Tool->GetSelection();
						if (Bones.Num() == 0)
						{
							return FText::FromName(NAME_None);
						}
						if (Bones.Num() == 1)
						{
							return FText::FromName(Bones[0]);
						}
						return MultiValues;
					})
					.OnTextCommitted_Lambda([this, NodePropHandle](const FText& NewText, ETextCommit::Type InCommitType) 
					{
						if (InCommitType == ETextCommit::OnCleared || NewText.EqualTo(MultiValues))
						{
							return;
						}
						FText CurrentText; NodePropHandle->GetValueAsFormattedText(CurrentText);
						if (!NewText.ToString().Equals(CurrentText.ToString(), ESearchCase::CaseSensitive))
						{
							NodePropHandle->SetValueFromFormattedString(NewText.ToString());
						}
					})
					.IsEnabled_Lambda([this] { return !Tool->GetSelection().IsEmpty(); })
			];
	}
	
	SAdvancedTransformInputBox<FTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FTransform>::FArguments()
		.IsEnabled(true)
		.DisplayRelativeWorld(true)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.DisplayScaleLock(false)
		.AllowEditRotationRepresentation(false);
	// relative/world
	TransformWidgetArgs
	.OnGetIsComponentRelative_Lambda( [this](ESlateTransformComponent::Type InComponent)
	{
	   return RelativeArray[InComponent];
	})
	.OnIsComponentRelativeChanged_Lambda( [this](ESlateTransformComponent::Type InComponent, bool bIsRelative)
	{
		RelativeArray[InComponent] = bIsRelative;
	});
	
	// get bones transforms
	CustomizeValueGet(TransformWidgetArgs);
		
	// set bones transforms
	CustomizeValueSet(TransformWidgetArgs);
	// copy/paste values
	CustomizeClipboard(TransformWidgetArgs);
	// enabled
	TransformWidgetArgs
	.IsEnabled_Lambda([this]()
	{
		return !Tool->GetSelection().IsEmpty();
	});
	
	SAdvancedTransformInputBox<FTransform>::ConstructGroupedTransformRows(
		CategoryBuilder, 
		LOCTEXT("ReferenceTransform", "Transform"),
		LOCTEXT("ReferenceBoneTransformTooltip", "The reference transform of the bone"), 
		TransformWidgetArgs);
	
	const TSharedRef<IPropertyHandle> Handle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkeletonEditingProperties, bUpdateChildren));
	if (IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(Handle))
	{
		Row->IsEnabled(IsEnabledBySelection());
	}
}

const TArray<FName>& FSkeletonEditingPropertiesDetailCustomization::GetProperties() const
{
	static const TArray< FName > Properties(
	{
		GET_MEMBER_NAME_CHECKED(USkeletonEditingProperties, Name),
		GET_MEMBER_NAME_CHECKED(USkeletonEditingProperties, bUpdateChildren)
	});
	return Properties;
}

void FSkeletonEditingPropertiesDetailCustomization::CustomizeValueGet(SAdvancedTransformInputBox<FTransform>::FArguments& InOutArgs)
{
	if (!Tool.IsValid())
	{
		return;
	}
	
	auto GetNumericValue = [this](
	   const FName InBoneName,
	   ESlateTransformComponent::Type InComponent,
	   ESlateRotationRepresentation::Type InRepresentation,
	   ESlateTransformSubComponent::Type InSubComponent)
	{
		const bool bWorld = !RelativeArray[InComponent];
		return SAdvancedTransformInputBox<FTransform>::GetNumericValueFromTransform(
			  Tool->GetTransform(InBoneName, bWorld),
			  InComponent,
			  InRepresentation,
			  InSubComponent);
	};

	InOutArgs.OnGetNumericValue_Lambda( [this, GetNumericValue](
		ESlateTransformComponent::Type InComponent,
		ESlateRotationRepresentation::Type InRepresentation,
		ESlateTransformSubComponent::Type InSubComponent)
	{
		const TArray<FName>& Bones = Tool->GetSelection();
		if (Bones.IsEmpty())
		{
			return SAdvancedTransformInputBox<FTransform>::GetNumericValueFromTransform(
				FTransform::Identity, InComponent, InRepresentation, InSubComponent);
		}
		
		TOptional<FVector::FReal> Value = GetNumericValue(Bones[0], InComponent, InRepresentation, InSubComponent);
		if (Value)
		{
			for (int32 Index = 1; Index < Bones.Num(); Index++)
			{
				const TOptional<FVector::FReal> NextValue = GetNumericValue(Bones[Index], InComponent, InRepresentation, InSubComponent);
				if (NextValue)
				{
					if (!FMath::IsNearlyEqual(*Value, *NextValue))
					{
						return TOptional<FVector::FReal>();
					}
				}
			}
		}
		return Value;
	} );
	
	InOutArgs.DiffersFromDefault_Lambda([this](ESlateTransformComponent::Type InComponent)
	{
		const TArray<FName>& Bones = Tool->GetSelection();
		for (const FName& BoneName: Bones)
		{
			const bool bWorld = !RelativeArray[InComponent];
			const FTransform& Transform = Tool->GetTransform(BoneName, bWorld);
			switch (InComponent)
			{
				case ESlateTransformComponent::Location:
					if (!Transform.GetLocation().Equals(FVector::ZeroVector))
					{
						return true;
					}
					break;
				case ESlateTransformComponent::Rotation:
					if(!Transform.GetRotation().Equals(FQuat::Identity))
					{
						return true;
					}
					break;
				case ESlateTransformComponent::Scale:
					if(!Transform.GetScale3D().Equals(FVector::OneVector))
					{
						return true;
					}
					break;
				default:
					if (!Transform.Equals(FTransform::Identity))
					{
						return true;
					}
					break;
			}
		}
		return false;
	});
}

void FSkeletonEditingPropertiesDetailCustomization::CustomizeValueSet(SAdvancedTransformInputBox<FTransform>::FArguments& InOutArgs)
{
	if (!Tool.IsValid())
	{
		return;
	}
	
	auto PrepareNumericValueChanged = [this]( const FName InBoneName,
											   ESlateTransformComponent::Type InComponent,
											   ESlateRotationRepresentation::Type InRepresentation,
											   ESlateTransformSubComponent::Type InSubComponent,
											   FTransform::FReal InValue)
	{
		const bool bWorld = !RelativeArray[InComponent];
		const FTransform& InTransform = Tool->GetTransform(InBoneName, bWorld);
		FTransform OutTransform = InTransform;
		SAdvancedTransformInputBox<FTransform>::ApplyNumericValueChange(OutTransform, InValue, InComponent, InRepresentation, InSubComponent);
		return MakeTuple(InTransform, OutTransform);
	};

	InOutArgs.OnNumericValueChanged_Lambda([this, PrepareNumericValueChanged](
		ESlateTransformComponent::Type InComponent,
		ESlateRotationRepresentation::Type InRepresentation,
		ESlateTransformSubComponent::Type InSubComponent,
		FVector::FReal InValue)
	{
		const TArray<FName>& Bones = Tool->GetSelection();
		TArray<FName> BonesToMove; BonesToMove.Reserve(Bones.Num());
		TArray<FTransform> UpdatedTransforms; UpdatedTransforms.Reserve(Bones.Num());
		
		FTransform CurrentTransform, UpdatedTransform;
		for (const FName& BoneName: Bones)
		{
			Tie(CurrentTransform, UpdatedTransform) =
				PrepareNumericValueChanged(BoneName, InComponent, InRepresentation, InSubComponent, InValue);
			if (!UpdatedTransform.Equals(CurrentTransform))
			{
				BonesToMove.Add(BoneName);
				UpdatedTransforms.Add(UpdatedTransform);
			}
		}

		if (!BonesToMove.IsEmpty())
		{
			if (!ActiveChange.IsValid())
			{
				ActiveChange = MakeUnique<SkeletonEditingTool::FRefSkeletonChange>(Tool.Get());
			}
		
			const bool bWorld = !RelativeArray[InComponent];
			Tool->SetTransforms(BonesToMove, UpdatedTransforms, bWorld);
		}
	});

	InOutArgs.OnNumericValueCommitted_Lambda([this, PrepareNumericValueChanged](
		ESlateTransformComponent::Type InComponent,
		ESlateRotationRepresentation::Type InRepresentation,
		ESlateTransformSubComponent::Type InSubComponent,
		FVector::FReal InValue,
		ETextCommit::Type InCommitType)
	{
		const TArray<FName>& Bones = Tool->GetSelection();

		TArray<FName> BonesToMove;
		BonesToMove.Reserve(Bones.Num());
		
		TArray<FTransform> UpdatedTransforms;
		UpdatedTransforms.Reserve(Bones.Num());
		
		FTransform CurrentTransform, UpdatedTransform;
		for (const FName& BoneName: Bones)
		{
			Tie(CurrentTransform, UpdatedTransform) =
				PrepareNumericValueChanged(BoneName, InComponent, InRepresentation, InSubComponent, InValue);
			if (!UpdatedTransform.Equals(CurrentTransform))
			{
				BonesToMove.Add(BoneName);
				UpdatedTransforms.Add(UpdatedTransform);
			}
		}

		if (!BonesToMove.IsEmpty())
		{
			if (!ActiveChange.IsValid())
			{
				ActiveChange = MakeUnique<SkeletonEditingTool::FRefSkeletonChange>(Tool.Get());
			}

			const bool bWorld = !RelativeArray[InComponent];
			Tool->SetTransforms(BonesToMove, UpdatedTransforms, bWorld);
		}

		if (ActiveChange.IsValid())
		{
			// send transaction
			if (UInteractiveToolManager* ToolManager = Tool->GetToolManager())
			{
				ActiveChange->StoreSkeleton(Tool.Get());

				static const FText TransactionDesc = LOCTEXT("ChangeNumericValue", "Change Numeric Value");
				ToolManager->BeginUndoTransaction(TransactionDesc);
				ToolManager->EmitObjectChange(Tool.Get(), MoveTemp(ActiveChange), TransactionDesc);
				ToolManager->EndUndoTransaction();
			}
		
			ActiveChange.Reset();
		}
	});

	auto PrepareNumericValueReset = [this]( const FName InBoneName, ESlateTransformComponent::Type InComponent)
	{
		const bool bWorld = !RelativeArray[InComponent];
		const FTransform& InTransform = Tool->GetTransform(InBoneName, bWorld);
		FTransform OutTransform = InTransform;

		switch (InComponent)
		{
		case ESlateTransformComponent::Location:
			OutTransform.SetLocation(FVector::ZeroVector);
			break;
		case ESlateTransformComponent::Rotation:
			OutTransform.SetRotation(FQuat::Identity);
			break;
		case ESlateTransformComponent::Scale:
			OutTransform.SetScale3D(FVector::OneVector);
			break;
		default:
			OutTransform = FTransform::Identity;
			break;
		}
		
		return MakeTuple(InTransform, OutTransform);
	};
	
	InOutArgs.OnResetToDefault_Lambda([this, PrepareNumericValueReset](ESlateTransformComponent::Type InComponent)
	{
		const TArray<FName>& Bones = Tool->GetSelection();

		TArray<FName> BonesToMove;
		BonesToMove.Reserve(Bones.Num());
		
		TArray<FTransform> UpdatedTransforms;
		UpdatedTransforms.Reserve(Bones.Num());
		
		FTransform CurrentTransform, UpdatedTransform;
		for (const FName& BoneName: Bones)
		{
			Tie(CurrentTransform, UpdatedTransform) = PrepareNumericValueReset(BoneName, InComponent);
			if (!UpdatedTransform.Equals(CurrentTransform))
			{
				BonesToMove.Add(BoneName);
				UpdatedTransforms.Add(UpdatedTransform);
			}
		}

		if (!BonesToMove.IsEmpty())
		{
			if (!ActiveChange.IsValid())
			{
				ActiveChange = MakeUnique<SkeletonEditingTool::FRefSkeletonChange>(Tool.Get());
			}

			const bool bWorld = !RelativeArray[InComponent];
			Tool->SetTransforms(BonesToMove, UpdatedTransforms, bWorld);
		}

		if (ActiveChange.IsValid())
		{
			// send transaction
			if (UInteractiveToolManager* ToolManager = Tool->GetToolManager())
			{
				ActiveChange->StoreSkeleton(Tool.Get());

				static const FText TransactionDesc = LOCTEXT("ResetNumericValue", "Reset Numeric Value");
				ToolManager->BeginUndoTransaction(TransactionDesc);
				ToolManager->EmitObjectChange(Tool.Get(), MoveTemp(ActiveChange), TransactionDesc);
				ToolManager->EndUndoTransaction();
			}
				
			ActiveChange.Reset();
		}
	});
}

namespace FSkeletonEditingToolClipboardLocals
{
	
template<typename DataType>
void GetContentFromData(const DataType& InData, FString& Content)
{
	TBaseStructure<DataType>::Get()->ExportText(Content, &InData, &InData, nullptr, PPF_None, nullptr);
}

class FSkeletonEditingToolBoneErrorPipe : public FOutputDevice
{
public:
	int32 NumErrors;

	FSkeletonEditingToolBoneErrorPipe()
		: FOutputDevice()
		, NumErrors(0)
	{}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		NumErrors++;
	}
};

template<typename DataType>
bool GetDataFromContent(const FString& Content, DataType& OutData)
{
	FSkeletonEditingToolBoneErrorPipe ErrorPipe;
	static UScriptStruct* DataStruct = TBaseStructure<DataType>::Get();
	DataStruct->ImportText(*Content, &OutData, nullptr, PPF_None, &ErrorPipe, DataStruct->GetName(), true);
	return (ErrorPipe.NumErrors == 0);
}
	
}

void FSkeletonEditingPropertiesDetailCustomization::CustomizeClipboard(SAdvancedTransformInputBox<FTransform>::FArguments& InOutArgs)
{
	if (!Tool.IsValid())
	{
		return;
	}
	
	auto OnCopyToClipboard = [this](const FName InBoneName, ESlateTransformComponent::Type InComponent)
	{
		using namespace FSkeletonEditingToolClipboardLocals;

		const bool bWorld = !RelativeArray[InComponent];
		const FTransform& Xfo = Tool->GetTransform(InBoneName, bWorld);

		FString Content;
		switch(InComponent)
		{
		case ESlateTransformComponent::Location:
			{
				GetContentFromData(Xfo.GetLocation(), Content);
				break;
			}
		case ESlateTransformComponent::Rotation:
			{
				GetContentFromData(Xfo.Rotator(), Content);
				break;
			}
		case ESlateTransformComponent::Scale:
			{
				GetContentFromData(Xfo.GetScale3D(), Content);
				break;
			}
		case ESlateTransformComponent::Max:
		default:
			{
				GetContentFromData(Xfo, Content);
				break;
			}
		}

		if (!Content.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*Content);
		}
	};

	auto PreparePasteFromClipboard = [this](const FName InBoneName, ESlateTransformComponent::Type InComponent)
	{
		using namespace FSkeletonEditingToolClipboardLocals;
	
		FString Content;
		FPlatformApplicationMisc::ClipboardPaste(Content);

		const bool bWorld = !RelativeArray[InComponent];
		FTransform Xfo = Tool->GetTransform(InBoneName, bWorld);
		
		if (Content.IsEmpty())
		{
			return MakeTuple(Xfo, Xfo);
		}

		switch(InComponent)
		{
		case ESlateTransformComponent::Location:
			{
				FVector Data = Xfo.GetLocation();
				if (GetDataFromContent(Content, Data))
				{
					Xfo.SetLocation(Data);
				}
				break;
			}
		case ESlateTransformComponent::Rotation:
			{
				FRotator Data = Xfo.Rotator();
				if (GetDataFromContent(Content, Data))
				{
					Xfo.SetRotation(FQuat(Data));
				}
				break;
			}
		case ESlateTransformComponent::Scale:
			{
				FVector Data = Xfo.GetScale3D();
				if (GetDataFromContent(Content, Data))
				{
					Xfo.SetScale3D(Data);
				}
				break;
			}
		case ESlateTransformComponent::Max:
		default:
			{
				FTransform Data = Xfo;
				if (GetDataFromContent(Content, Data))
				{
					Xfo = Data;
				}
				break;
			}
		}

		return MakeTuple(Tool->GetTransform(InBoneName, bWorld), Xfo);
	};

	InOutArgs.OnCopyToClipboard_Lambda( [this, OnCopyToClipboard](ESlateTransformComponent::Type InComponent)
	{
		const TArray<FName>& Bones = Tool->GetSelection();
		if (!Bones.IsEmpty())
		{
			OnCopyToClipboard(Bones[0], InComponent);
		}
	})
	.OnPasteFromClipboard_Lambda([this, PreparePasteFromClipboard](ESlateTransformComponent::Type InComponent)
	{
		const TArray<FName>& Bones = Tool->GetSelection();
		TArray<FName> BonesToMove; BonesToMove.Reserve(Bones.Num());
		TArray<FTransform> UpdatedTransforms; UpdatedTransforms.Reserve(Bones.Num());
		
		FTransform CurrentTransform, UpdatedTransform;
		for (const FName& BoneName: Bones)
		{
			Tie(CurrentTransform, UpdatedTransform) = PreparePasteFromClipboard(BoneName, InComponent);
			if (!UpdatedTransform.Equals(CurrentTransform))
			{
				BonesToMove.Add(BoneName);
				UpdatedTransforms.Add(UpdatedTransform);
			}
		}

		if (BonesToMove.IsEmpty())
		{
			return;
		}
		
		if (!ActiveChange.IsValid())
		{
			ActiveChange = MakeUnique<SkeletonEditingTool::FRefSkeletonChange>(Tool.Get());
		}
		
		const bool bWorld = !RelativeArray[InComponent];
		Tool->SetTransforms(BonesToMove, UpdatedTransforms, bWorld);

		// send transaction
		if (UInteractiveToolManager* ToolManager = Tool->GetToolManager())
		{
			ActiveChange->StoreSkeleton(Tool.Get());
			
			static const FText TransactionDesc = LOCTEXT("PasteTransform", "Paste Transform");
			ToolManager->BeginUndoTransaction(TransactionDesc);
			ToolManager->EmitObjectChange(Tool.Get(), MoveTemp(ActiveChange), TransactionDesc);
			ToolManager->EndUndoTransaction();
		}
		ActiveChange.Reset();
	});
}

TSharedRef<IDetailCustomization> FMirroringPropertiesDetailCustomization::MakeInstance()
{
	return MakeShareable(new FMirroringPropertiesDetailCustomization);
}

const TArray<FName>& FMirroringPropertiesDetailCustomization::GetProperties() const
{
	static const TArray< FName > Properties(
	{
		GET_MEMBER_NAME_CHECKED(UMirroringProperties, Options)
	});
	return Properties;
}

void FMirroringPropertiesDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UE::SkeletalMeshModelingTools;

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FMirrorOptions::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSKMMTMirroringPropertiesAxisCustomization::MakeInstance));

	const TWeakObjectPtr<UMirroringProperties> Properties = GetCustomizedObject<UMirroringProperties>(DetailBuilder);
	if (!GetParentTool(Properties).IsValid())
	{
		return;
	}
	const TAttribute<EVisibility> EditionVisibility = GetEditionVisibility();
	
	UpdateProperties(DetailBuilder, EditionVisibility, IsEnabledBySelection());
	
	static const FName MirrorCategoryName("Mirror");
	IDetailCategoryBuilder& MirrorCategory = DetailBuilder.EditCategory(MirrorCategoryName, FText::GetEmpty(), ECategoryPriority::Important);
	
	MirrorCategory.AddCustomRow(LOCTEXT("MirrorButtonRow", "Mirror"))
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(2.f, 4.f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("MirrorButtonLabel", "Mirror"))
			.OnClicked_Lambda([Properties]()
			{
				Properties->MirrorBones();
				return FReply::Handled();
			})
		]
	]
	.Visibility(EditionVisibility)
	.IsEnabled(IsEnabledBySelection());
}

TSharedRef<IDetailCustomization> FOrientingPropertiesDetailCustomization::MakeInstance()
{
	return MakeShareable(new FOrientingPropertiesDetailCustomization);
}

const TArray<FName>& FOrientingPropertiesDetailCustomization::GetProperties() const
{
	static const TArray< FName > Properties(
	{
		GET_MEMBER_NAME_CHECKED(UOrientingProperties, Options),
		GET_MEMBER_NAME_CHECKED(UOrientingProperties, bAutoOrient)
	});
	return Properties;
}

void FOrientingPropertiesDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UE::SkeletalMeshModelingTools;

	DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(
		FOrientOptions::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSKMMTOrientPropertiesAxisCustomization::MakeInstance));

	TWeakObjectPtr<UOrientingProperties> Properties = GetCustomizedObject<UOrientingProperties>(DetailBuilder);
	if (!GetParentTool(Properties).IsValid())
	{
		return;
	}
	const TAttribute<EVisibility> EditionVisibility = GetEditionVisibility();
	
	UpdateProperties(DetailBuilder, EditionVisibility, IsEnabledBySelection());
	static const FName OrientCategoryName("Orient");
	IDetailCategoryBuilder& OrientCategory = DetailBuilder.EditCategory(OrientCategoryName, FText::GetEmpty(), ECategoryPriority::Important);
	OrientCategory.AddCustomRow(LOCTEXT("OrientButtonRow", "Orient"), false)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(2.f, 4.f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("OrientButtonLabel", "Orient"))
			.IsEnabled_Lambda([Properties]()
			{
				return Properties->Options.Primary != EOrientAxis::None; 
			})
			.OnClicked_Lambda([Properties]()
			{
				Properties->OrientBones();
				return FReply::Handled();
			})
		]
	]
	.Visibility(EditionVisibility)
	.IsEnabled(IsEnabledBySelection());
}

TSharedRef<IDetailCustomization> FProjectionPropertiesDetailCustomization::MakeInstance()
{
	return MakeShareable(new FProjectionPropertiesDetailCustomization);
}

const TArray<FName>& FProjectionPropertiesDetailCustomization::GetProperties() const
{
	static const TArray< FName > Properties(
	{
		GET_MEMBER_NAME_CHECKED(UProjectionProperties, ProjectionType)
	});
	return Properties;
}

void FProjectionPropertiesDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const TWeakObjectPtr<UProjectionProperties> Properties = GetCustomizedObject<UProjectionProperties>(DetailBuilder);
	if (!GetParentTool(Properties).IsValid())
	{
		return;
	}
	UpdateProperties(DetailBuilder, GetCreationVisibility());
}

namespace SkeletonEditingCustomizationLocals
{
	enum EOperationType
	{
		Add,
		Edit,
		Component
	};
}

TSharedRef<IDetailCustomization> FSkeletonEditingToolDetailCustomization::MakeInstance()
{
	return MakeShareable(new FSkeletonEditingToolDetailCustomization);
}

void FSkeletonEditingToolDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	
	Tool = ObjectsBeingCustomized.Num() == 1 ?
		CastChecked<USkeletonEditingTool>(ObjectsBeingCustomized[0]) : nullptr;
	
	if (!Tool.IsValid())
	{
		return;
	}

	const FSegmentedControlStyle& SegmentedControlStyle = FAppStyle::Get().GetWidgetStyle<FSegmentedControlStyle>("SegmentedControl");
	
	auto IsCreateEnabled = [this]()
	{
		return Tool->GetOperation() == EEditingOperation::Create;
	};
	using namespace SkeletonEditingCustomizationLocals;
	
	IDetailCategoryBuilder& ActionCategory = DetailBuilder.EditCategory("Action", FText::GetEmpty(), ECategoryPriority::Important);
	ActionCategory.AddCustomRow(LOCTEXT("ActionCategory", "Action"), false)
	[
		SNew(SBorder)
		.BorderImage(&SegmentedControlStyle.BackgroundBrush)
		.Padding(FMargin(0,0,SegmentedControlStyle.UniformPadding.Right,0))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				.Style(&SegmentedControlStyle.FirstControlStyle)
				.ToolTipText(LOCTEXT("AddModeTooltip", "Create new bones. (N)"))
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckState)
				{
					if (InCheckState == ECheckBoxState::Checked)
					{
						Tool->SetOperation(EEditingOperation::Create);
					}
				})
				.IsChecked_Lambda([IsCreateEnabled]()
				{
					return IsCreateEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddMode", "Add"))
						.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("ButtonText"))
						.ColorAndOpacity(FLinearColor::White)
					]
				]
			]
			+SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				.Style(&SegmentedControlStyle.LastControlStyle)
				.ToolTipText(LOCTEXT("EditModeTooltip", "Edit current bone(s) selection. (Esc)"))
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckState)
				{
					if (InCheckState == ECheckBoxState::Checked)
					{
						Tool->SetOperation(EEditingOperation::Select);
					}
				})
				.IsChecked_Lambda([IsCreateEnabled]()
				{
					return IsCreateEnabled() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
				})
				[
					SNew(SOverlay)
						
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("EditMode", "Edit"))
						.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("ButtonText"))
						.ColorAndOpacity(FLinearColor::White)
					]
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.ToolTipText(LOCTEXT("ComponentModeTooltip", "Select vertices, edges, and triangles to place bone. (T)"))
				.OnCheckStateChanged_Lambda([this](ECheckBoxState)
				{
					Tool->Properties->bEnableComponentSelection = !Tool->Properties->bEnableComponentSelection;
				})
				.IsChecked_Lambda([this]()
				{
					return Tool->Properties->bEnableComponentSelection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				[
					SNew(SImage)
						.Image(FModelingToolsEditorModeStyle::Get()->GetBrush("ModelingToolsManagerCommands.BeginMeshSelectionTool"))
				]
			]
		]
	];

	CustomizeEditAction(ActionCategory);
	CustomizeComponentSelection(DetailBuilder);
}

void FSkeletonEditingToolDetailCustomization::CustomizeComponentSelection(IDetailLayoutBuilder& DetailBuilder) const
{
	const TAttribute<bool> IsEnabledBySelection = TAttribute<bool>::CreateLambda([this]()
	{
		return Tool.IsValid() ? Tool->HasSelectedComponent() && !Tool->GetSelection().IsEmpty() : false;
	});

	const TAttribute<EVisibility> VisibilityAttribute = TAttribute<EVisibility>::CreateLambda([this]()
	{
		return (Tool.IsValid() && Tool->Properties->bEnableComponentSelection) ? EVisibility::Visible : EVisibility::Hidden;
	});

	static const FName SnapCategoryName("Component Snapping");
	IDetailCategoryBuilder& SnapCategory = DetailBuilder.EditCategory(SnapCategoryName, FText::GetEmpty(), ECategoryPriority::Important);

	SnapCategory.AddCustomRow(LOCTEXT("SnapButtonRow", "Snap"), false)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(2.f, 4.f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text_Lambda([this]
			{
				const bool bCreate = Tool.IsValid() && Tool->GetOperation() == EEditingOperation::Create;
				return bCreate ? LOCTEXT("SnapCreateButtonLabel", "Create") : LOCTEXT("SnapButtonLabel", "Snap");
			})
			.ToolTipText_Lambda([this]
			{
				const bool bCreate = Tool.IsValid() && Tool->GetOperation() == EEditingOperation::Create;
				return bCreate ? LOCTEXT("SnapCreateButtonTooltip", "Create a new bone and snap it to the selected components. (V)") :
								LOCTEXT("SnapButtonTooltip", "Snap the selected bone to the selected components. (V)");
			})
			.IsEnabled_Lambda([this]()
			{
				if (!Tool.IsValid())
				{
					return false;
				}
				const bool bCreate = Tool->GetOperation() == EEditingOperation::Create;
				return bCreate ? Tool->HasSelectedComponent() : Tool->HasSelectedComponent() && !Tool->GetSelection().IsEmpty(); 
			})
			.OnClicked_Lambda([this]()
			{
				const bool bCreate = Tool.IsValid() && Tool->GetOperation() == EEditingOperation::Create;
				Tool->SnapBoneToComponentSelection(bCreate);
				return FReply::Handled();
			})
		]
	]
	.Visibility(VisibilityAttribute);
	
	// add selection properties
	TObjectPtr<UMeshTopologySelectionMechanicProperties> SelectionMechanicProperties = Tool->SelectionMechanic->Properties;
	if (IDetailPropertyRow* SelectionMechanicRow = SnapCategory.AddExternalObjects({SelectionMechanicProperties}))
	{
		TSharedPtr<SWidget> NameWidget, ValueWidget;
		SelectionMechanicRow->GetDefaultWidgets(NameWidget, ValueWidget);
		SelectionMechanicRow->CustomWidget()
		.NameContent()
		[
			SNew(STextBlock)
				.Text(FText::FromString(TEXT("Options")))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			ValueWidget->AsShared()
		];
		SelectionMechanicRow->ShouldAutoExpand(false);
		SelectionMechanicRow->Visibility(VisibilityAttribute);
	}
}

void FSkeletonEditingToolDetailCustomization::CustomizeEditAction(IDetailCategoryBuilder& InActionCategory) const
{
	if (!Tool.IsValid())
	{
		return;
	}
	
	const TAttribute<EVisibility> EditionVisibility = TAttribute<EVisibility>::CreateLambda([this]()
	{
		if (!Tool.IsValid())
		{
			return EVisibility::Hidden;
		}
		return Tool->GetOperation() == EEditingOperation::Create ? EVisibility::Hidden : EVisibility::Visible;
	});
	
	const TAttribute<bool> IsEnabledBySelection = TAttribute<bool>::CreateLambda([this]()
	{
		return Tool.IsValid() ? !Tool->GetSelection().IsEmpty() : false;
	});
	FSegmentedControlStyle SegmentedControlStyle = FAppStyle::Get().GetWidgetStyle<FSegmentedControlStyle>("SegmentedControl");
	
	InActionCategory.AddCustomRow(LOCTEXT("EditCategory", "Edit"), false)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(2.f, 4.f)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "SegmentedCombo.ButtonOnly")
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckState)
			{
				const EEditingOperation Operation = InCheckState == ECheckBoxState::Checked ?
					EEditingOperation::Parent : EEditingOperation::Select;
				Tool->SetOperation(Operation);
			})
			.IsChecked_Lambda([this]()
			{
				return Tool->GetOperation() == EEditingOperation::Parent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.IsEnabled(IsEnabledBySelection)
			[
				SNew(SOverlay)
				
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EditParent", "Reparent"))
					.ToolTipText(LOCTEXT("EditParentTooltip", "Click on a bone to be set as the new parent. (B)"))
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("ButtonText"))
					.ColorAndOpacity(FLinearColor::White)
				]
			]
		]
		+SHorizontalBox::Slot()
		.Padding(2.f, 4.f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("EditDisconnect", "Disconnect"))
			.ToolTipText(LOCTEXT("EditDisconnectTooltip", "Unparent current selection. (SHIFT+P)"))
			.OnClicked_Lambda([this]()
			{
				Tool->UnParentBones();
				return FReply::Handled();
			})
			.IsEnabled_Lambda([this]()
			{
				if (!Tool.IsValid())
				{
					return false;
				}
				return !Tool->GetSelection().IsEmpty() && Tool->GetOperation() != EEditingOperation::Parent;
			})
		]
		
		+SHorizontalBox::Slot()
		.Padding(2.f, 4.f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("EditRemove", "Remove"))
			.ToolTipText(LOCTEXT("EditRemoveTooltip", "Remove current selection. (Delete)"))
			.OnClicked_Lambda([this]()
			{
				Tool->RemoveBones();
				return FReply::Handled();
			})
			.IsEnabled_Lambda([this]()
			{
				if (!Tool.IsValid())
				{
					return false;
				}
				return !Tool->GetSelection().IsEmpty() && Tool->GetOperation() != EEditingOperation::Parent;
			})
		]
	]
	.Visibility(EditionVisibility);
}

#undef LOCTEXT_NAMESPACE