// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "SAdvancedTransformInputBox.h"
#include "SkeletalMesh/SkeletonEditingTool.h"

class IDetailLayoutBuilder;

/**
 * FSkeletonEditingToolDetailCustomization
 */

class FSkeletonEditingToolDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
private:
	void CustomizeEditAction(IDetailCategoryBuilder& InActionCategory) const;
	void CustomizeComponentSelection(IDetailLayoutBuilder& DetailBuilder) const;
	
	TWeakObjectPtr<USkeletonEditingTool> Tool = nullptr;
};

/**
 * ISkeletonEditingPropertiesDetailCustomization
 */

class ISkeletonEditingPropertiesDetailCustomization : public IDetailCustomization
{
protected:
	template< typename T >
	static TWeakObjectPtr<T> GetCustomizedObject(const IDetailLayoutBuilder& InDetailBuilder)
	{
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
		if (ObjectsBeingCustomized.Num() != 1)
		{
			return nullptr;
		}
		return CastChecked<T>(ObjectsBeingCustomized[0]);
	}
	
	template<typename TProperty>
	TWeakObjectPtr<USkeletonEditingTool> GetParentTool(const TWeakObjectPtr<TProperty>& InProperties)
	{
		Tool = InProperties.IsValid() ? InProperties->ParentTool : nullptr;	
		return Tool;
	}

	TAttribute<EVisibility> GetCreationVisibility() const;
	TAttribute<EVisibility> GetEditionVisibility() const;
	TAttribute<bool> IsEnabledBySelection() const;
	
	void UpdateProperties(
		IDetailLayoutBuilder& InDetailBuilder,
		const TAttribute<EVisibility>& InVisibility,
		const TAttribute<bool>& InEnabled = TAttribute<bool>()) const;
	
	virtual const TArray<FName>& GetProperties() const = 0;
	
	TWeakObjectPtr<USkeletonEditingTool> Tool;
};

/**
 * FSkeletonEditingPropertiesDetailCustomization
 */

class FSkeletonEditingPropertiesDetailCustomization : public ISkeletonEditingPropertiesDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	virtual const TArray<FName>& GetProperties() const override;
	
private:
	void CustomizeValueGet(SAdvancedTransformInputBox<FTransform>::FArguments& InOutArgs);
	void CustomizeValueSet(SAdvancedTransformInputBox<FTransform>::FArguments& InOutArgs);
	void CustomizeClipboard(SAdvancedTransformInputBox<FTransform>::FArguments& InOutArgs);
	
	TUniquePtr<SkeletonEditingTool::FRefSkeletonChange> ActiveChange;
	TBitArray<> RelativeArray;
};

/**
 * FMirroringPropertiesDetailCustomization
 */

class FMirroringPropertiesDetailCustomization : public ISkeletonEditingPropertiesDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	virtual const TArray<FName>& GetProperties() const override;
};

/**
 * FOrientingPropertiesDetailCustomization
 */

class FOrientingPropertiesDetailCustomization : public ISkeletonEditingPropertiesDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	virtual const TArray<FName>& GetProperties() const override;
};

/**
 * FProjectionPropertiesDetailCustomization
 */

class FProjectionPropertiesDetailCustomization : public ISkeletonEditingPropertiesDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
protected:
	virtual const TArray<FName>& GetProperties() const override;
};

