// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinWeightDetailCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "IDetailPropertyRow.h"
#include "ModelingToolsEditorModeStyle.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SkeletalMesh/SReferenceSkeletonTree.h"
#include "SkeletalMeshModelingToolsStyle.h"

#define LOCTEXT_NAMESPACE "SkinWeightToolSettingsEditor"

// layout constants
float FSkinWeightDetailCustomization::WeightSliderWidths = 150.0f;
float FSkinWeightDetailCustomization::WeightEditingLabelsPercent = 0.40f;
float FSkinWeightDetailCustomization::WeightEditVerticalPadding = 4.0f;
float FSkinWeightDetailCustomization::WeightEditHorizontalPadding = 2.0f;

static FName ColumnName_Bone = "Bone";
static FName ColumnName_Weight = "Weight";
static FName ColumnName_Prune = "Prune";

FSkinWeightDetailCustomization::~FSkinWeightDetailCustomization()
{
	if (Tool.IsValid())
	{
		Tool->OnSelectionChanged.RemoveAll(this);
	}

	Tool.Reset();
	ToolSettings.Reset();
}

void FSkinWeightDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TWeakPtr<FSkinWeightDetailCustomization> WeakThis = SharedThis(this);
	CurrentDetailBuilder = &DetailBuilder;
	
	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);

	// should be impossible to get multiple settings objects for a single tool
	ensure(DetailObjects.Num()==1);
	ToolSettings = Cast<USkinWeightsPaintToolProperties>(DetailObjects[0]);
	Tool = ToolSettings->WeightTool;
	Tool->OnSelectionChanged.AddSP(this, &FSkinWeightDetailCustomization::OnSelectionChanged);
	
	// custom display of falloff mode as segmented toggle buttons
	IDetailCategoryBuilder& EditModeCategory = DetailBuilder.EditCategory("Weight Editing Mode", FText::GetEmpty(), ECategoryPriority::Important);

	// add segmented control toggle for editing modes ("Brush" or "Selection")
	EditModeCategory.AddCustomRow(LOCTEXT("EditModeCategory", "Weight Editing Mode"), false)
	.WholeRowContent()
	[
		SNew(SBox)
		.Padding(2.0f)
		[
			SNew(SSegmentedControl<EWeightEditMode>)
			.ToolTipText(LOCTEXT("EditingModeTooltip",
					"Brush: edit weights by painting on mesh.\n"
					"Mesh: select vertices/edges/faces to edit weights directly.\n"
					"Bones: select and manipulate bones to preview deformations.\n"))
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->EditingMode : EWeightEditMode::Brush;
			})
			.OnValueChanged_Lambda([this](EWeightEditMode Mode)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->EditingMode = Mode;
					ToolSettings->WeightTool->ToggleEditingMode();
					if (CurrentDetailBuilder)
					{
						CurrentDetailBuilder->ForceRefreshDetails();
					}
				}
			})
			+SSegmentedControl<EWeightEditMode>::Slot(EWeightEditMode::Brush)
			.Text(LOCTEXT("BrushEditMode", "Brush"))
			+ SSegmentedControl<EWeightEditMode>::Slot(EWeightEditMode::Mesh)
			.Text(LOCTEXT("MeshEditMode", "Mesh"))
			+ SSegmentedControl<EWeightEditMode>::Slot(EWeightEditMode::Bones)
			.Text(LOCTEXT("BoneEditMode", "Bones"))
		]
	];

	// BRUSH editing mode UI
	if (ToolSettings->EditingMode == EWeightEditMode::Brush)
	{
		AddBrushUI(DetailBuilder);
	}

	// MESH editing mode UI
	if (ToolSettings->EditingMode == EWeightEditMode::Mesh)
	{
		AddSelectionUI(DetailBuilder);
	}
	
	// COLOR MODE category
	IDetailCategoryBuilder& MeshDisplayCategory = DetailBuilder.EditCategory("MeshDisplay", FText::GetEmpty(), ECategoryPriority::Important);
	MeshDisplayCategory.InitiallyCollapsed(false);
	MeshDisplayCategory.AddCustomRow(LOCTEXT("ColorModeCategory", "Color Mode"), false)
	.WholeRowContent()
	[
		SNew(SBox)
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SSegmentedControl<EWeightColorMode>)
			.ToolTipText(LOCTEXT("WeightColorTooltip",
					"Adjust the weight display in the viewport.\n\n"
					"Greyscale: Displays weights on the current bone by blending from black (0) to white (1).\n"
					"Ramp: Displays weights on the current bone. Weights at 0 and 1 use the min and max colors. Weights inbetween 0 and 1 use the ramp colors.\n"
					"Multi Color: Displays weights on ALL bones using the color of the bones.\n"
					"Full Material: Displays normal mesh materials with textures.\n"))
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->ColorMode : EWeightColorMode::Greyscale;
			})
			.OnValueChanged_Lambda([this](EWeightColorMode Mode)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->SetColorMode(Mode);
				}
			})
			+ SSegmentedControl<EWeightColorMode>::Slot(EWeightColorMode::Greyscale)
			.Text(LOCTEXT("GreyscaleMode", "Greyscale"))
			+SSegmentedControl<EWeightColorMode>::Slot(EWeightColorMode::Ramp)
			.Text(LOCTEXT("RampMode", "Ramp"))
			+SSegmentedControl<EWeightColorMode>::Slot(EWeightColorMode::BoneColors)
			.Text(LOCTEXT("BoneColorsMode", "Bone Colors"))
			+SSegmentedControl<EWeightColorMode>::Slot(EWeightColorMode::FullMaterial)
			.Text(LOCTEXT("MaterialMode", "Full Material"))
		]
	];

	// Mesh Element Display category
	IDetailCategoryBuilder& MeshElementDisplayCategory =
		DetailBuilder.EditCategory("MeshElementDisplay", LOCTEXT("MeshElementDisplayCategory", "Mesh Element Display"), ECategoryPriority::Important);

	auto MakeOnPropertyChangedDelegate = [this, WeakThis](TFunction<void(USkinWeightsPaintToolProperties&, UMeshElementsVisualizerProperties&)> ApplyFn)
	{
		return FSimpleDelegate::CreateLambda([this, WeakThis, ApplyFn]()
		{
			if (!WeakThis.IsValid())
			{
				return;
			}
			USkinWeightsPaintToolProperties* Settings = ToolSettings.Get();
			USkinWeightsPaintTool* WeightTool = Tool.Get();
			if (!Settings)
			{
				return;
			}

			UMeshElementsVisualizer* Visualizer = WeightTool ? WeightTool->GetMeshElementsDisplay() : nullptr;
			if (Visualizer && Visualizer->Settings)
			{
				ApplyFn(*Settings, *Visualizer->Settings);
			}
			Settings->SaveConfig();
		});
	};

	TSharedRef<IPropertyHandle> ShowGroupBoundariesHandle =
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, bShowGroupBoundaries));
	MeshElementDisplayCategory.AddProperty(ShowGroupBoundariesHandle)
		.DisplayName(LOCTEXT("ShowPolygroupEdgesLabel", "Show Polygroup Edges"));
	ShowGroupBoundariesHandle->SetOnPropertyValueChanged(MakeOnPropertyChangedDelegate(
		[](USkinWeightsPaintToolProperties& ToolProperties, UMeshElementsVisualizerProperties& VisualizerProperties)
		{
			VisualizerProperties.bVisible = ToolProperties.bShowGroupBoundaries;
		}));

	TSharedRef<IPropertyHandle> GroupBoundaryColorHandle =
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, GroupBoundaryColor));
	MeshElementDisplayCategory.AddProperty(GroupBoundaryColorHandle)
		.DisplayName(LOCTEXT("PolygroupEdgeColorLabel", "Polygroup Edge Color"));
	GroupBoundaryColorHandle->SetOnPropertyValueChanged(MakeOnPropertyChangedDelegate(
	[](USkinWeightsPaintToolProperties& ToolProperties, UMeshElementsVisualizerProperties& VisualizerProperties)
		{
			VisualizerProperties.GroupBoundaryColor = ToolProperties.GroupBoundaryColor;
		}));

	TSharedRef<IPropertyHandle> GroupBoundaryThicknessHandle =
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, GroupBoundaryThickness));
	MeshElementDisplayCategory.AddProperty(GroupBoundaryThicknessHandle)
		.DisplayName(LOCTEXT("PolygroupEdgeThicknessLabel", "Polygroup Edge Thickness"));
	GroupBoundaryThicknessHandle->SetOnPropertyValueChanged(MakeOnPropertyChangedDelegate(
	[](USkinWeightsPaintToolProperties& ToolProperties, UMeshElementsVisualizerProperties& VisualizerProperties)
		{
			VisualizerProperties.ThicknessScale = ToolProperties.GroupBoundaryThickness;
		}));

	AddTransferUI(DetailBuilder);

	// Edit SkinWeightLayer category 
	IDetailCategoryBuilder& SkinWeightLayerCategory = DetailBuilder.EditCategory("SkinWeightLayer", FText::GetEmpty(), ECategoryPriority::Important);
	SkinWeightLayerCategory.InitiallyCollapsed(true);
	
	// hide skin weight tool properties that were customized
	TArray<FName> PropertiesToHide;
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, BrushMode));
	
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, EditingMode));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, ColorMode));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, MeshSelectMode));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, bShowGroupBoundaries));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, GroupBoundaryColor));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, GroupBoundaryThickness));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, SourceSkeletalMesh));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, SourceLOD));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, SourceSkinWeightProfile));
	for (const FName PropertyToHide : PropertiesToHide)
	{
		TSharedRef<IPropertyHandle> Property = DetailBuilder.GetProperty(PropertyToHide);
		DetailBuilder.HideProperty(Property);
	}
	// hide base class properties
	PropertiesToHide.Reset();
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, bSpecifyRadius));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushSize));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushStrength));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushFalloffAmount));
	PropertiesToHide.Add(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushRadius));
	for (const FName PropertyToHide : PropertiesToHide)
	{
		TSharedRef<IPropertyHandle> Property = DetailBuilder.GetProperty(PropertyToHide, UBrushBaseProperties::StaticClass());
		DetailBuilder.HideProperty(Property);
	}
}

void FSkinWeightDetailCustomization::AddBrushUI(IDetailLayoutBuilder& DetailBuilder) const
{
	// custom display of falloff mode as segmented toggle buttons
	IDetailCategoryBuilder& BrushCategory = DetailBuilder.EditCategory("Brush", FText::GetEmpty(), ECategoryPriority::Important);

	// add segmented control toggle for brush behavior modes ("Add", "Replace", etc..)
	BrushCategory.AddCustomRow(LOCTEXT("BrushModeCategory", "Brush Mode"), false)
	.WholeRowContent()
	[
		SNew(SBox)
		.Padding(2.0f)
		[
			SNew(SSegmentedControl<EWeightEditOperation>)
			.ToolTipText(LOCTEXT("BrushTooltip",
				"Select the operation to apply when using the brush.\n"
				"Add: applies the current weight plus the Strength value to the new weight.\n"
				"Replace: applies the current weight minus the Strength value to the new weight.\n"
				"Multiply: applies the current weight multiplied by the Strength value to the new weight.\n"
				"Relax: applies the average of the connected (by edge) vertex weights to the new vertex weight, blended by the Strength."))
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->BrushMode : EWeightEditOperation::Add;
			})
			.OnValueChanged_Lambda([this](EWeightEditOperation Mode)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->SetBrushMode(Mode);
				}
			})
			+SSegmentedControl<EWeightEditOperation>::Slot(EWeightEditOperation::Add)
			.Text(LOCTEXT("BrushAddMode", "Add"))
			+ SSegmentedControl<EWeightEditOperation>::Slot(EWeightEditOperation::Replace)
			.Text(LOCTEXT("BrushReplaceMode", "Replace"))
			+ SSegmentedControl<EWeightEditOperation>::Slot(EWeightEditOperation::Multiply)
			.Text(LOCTEXT("BrushMultiplyMode", "Multiply"))
			+ SSegmentedControl<EWeightEditOperation>::Slot(EWeightEditOperation::Relax)
			.Text(LOCTEXT("BrushRelaxMode", "Relax"))
		]
	];

	// add segmented control toggle for brush falloff modes ("Surface" or "Volume")
	BrushCategory.AddCustomRow(LOCTEXT("BrushFalloffModeCategory", "Brush Falloff Mode"), false)
	.WholeRowContent()
	[
		SNew(SBox)
		.Padding(2.0f)
		[
			SNew(SSegmentedControl<EWeightBrushFalloffMode>)
			.ToolTipText(LOCTEXT("BrushFalloffModeTooltip",
					"Surface: falloff is based on the distance along the surface from the brush center to nearby connected vertices.\n"
					"Volume: falloff is based on the straight-line distance from the brush center to surrounding vertices.\n"))
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->GetBrushConfig().FalloffMode : EWeightBrushFalloffMode::Surface;
			})
			.OnValueChanged_Lambda([this](EWeightBrushFalloffMode Mode)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->SetFalloffMode(Mode);
				}
			})
			+SSegmentedControl<EWeightBrushFalloffMode>::Slot(EWeightBrushFalloffMode::Surface)
			.Text(LOCTEXT("SurfaceMode", "Surface"))
			+ SSegmentedControl<EWeightBrushFalloffMode>::Slot(EWeightBrushFalloffMode::Volume)
			.Text(LOCTEXT("VolumeMode", "Volume"))
		]
	];

	BrushCategory.AddCustomRow(LOCTEXT("BrushSizeCategory", "Brush Radius"), false)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("BrushRadiusLabel", "Radius"))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.ToolTipText(LOCTEXT("BrushRadiusTooltip", "The radius of the brush in scene units."))
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(2.f)
		.FillWidth(1.0f)
		[
			SNew(SSpinBox<float>)
			.MinValue(0.01f)
			.MaxSliderValue(20.f)
			.Value(10.0f)
			.SupportDynamicSliderMaxValue(true)
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->GetBrushConfig().Radius : 20.f;
			})
			.OnValueChanged_Lambda([this](float NewValue)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->BrushRadius = NewValue;
					// Honour the user's Sync toggle: lockstep across modes when on,
					// per-mode save when off. OnPropertyModified will also see this via
					// PostEditChangeProperty and applies the same gate; the explicit write
					// here keeps the live slider responsive without depending on that round-trip.
					if (ToolSettings->bSyncBrushRadiusAcrossModes)
					{
						ToolSettings->SetSharedRadius(NewValue);
					}
					else
					{
						ToolSettings->GetBrushConfig().Radius = NewValue;
					}
					FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushRadius)));
					ToolSettings->PostEditChangeProperty(PropertyChangedEvent);
				}
			})
			.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->SaveConfig();
				}
			})
		]
		+ SHorizontalBox::Slot()
		.Padding(2.f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.IsChecked_Lambda([this]() -> ECheckBoxState
			{
				const bool bSync = ToolSettings.IsValid() && ToolSettings->bSyncBrushRadiusAcrossModes;
				return bSync ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this](const ECheckBoxState NewState)
			{
				if (!ToolSettings.IsValid()) return;
				const bool bNewSync = (NewState == ECheckBoxState::Checked);
				ToolSettings->bSyncBrushRadiusAcrossModes = bNewSync;
				// Snap-on-enable: when re-linking, push the live radius to every per-mode config
				// so toggling on never silently drops the active value.
				if (bNewSync)
				{
					ToolSettings->SetSharedRadius(ToolSettings->BrushRadius);
				}
				ToolSettings->SaveConfig();
			})
			.Padding(2.0f)
			.ToolTipText(LOCTEXT("SyncBrushRadiusTooltip",
				"When linked, the brush radius is shared across all brush modes. "
				"When unlinked, each mode remembers its own radius."))
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image_Lambda([this]()
				{
					const bool bSync = ToolSettings.IsValid() && ToolSettings->bSyncBrushRadiusAcrossModes;
					return FAppStyle::GetBrush(bSync ? "Icons.Link" : "Icons.Unlink");
				})
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	BrushCategory.AddCustomRow(LOCTEXT("BrushStrengthCategory", "Brush Strength"), false)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("BrushStrengthLabel", "Strength"))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.ToolTipText(LOCTEXT("BrushStrengthTooltip", "The strength of the effect on the weights. Exact effect depends on the Brush mode."))
	]
	.ValueContent()
	[
		SNew(SSpinBox<float>)
		.MinValue(0.f)
		.MaxValue(1.0f)
		.MaxSliderValue(1.0f)
		.Value(1.0f)
		.Value_Lambda([this]()
		{
			return ToolSettings.IsValid() ? ToolSettings->GetBrushConfig().Strength : 1.f;
		})
		.OnValueChanged_Lambda([this](float NewValue)
		{
			if (ToolSettings.IsValid())
			{
				ToolSettings->BrushStrength = NewValue;
				ToolSettings->GetBrushConfig().Strength = NewValue;
				FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushStrength)));
				ToolSettings->PostEditChangeProperty(PropertyChangedEvent);
			}
		})
		.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
		{
			if (ToolSettings.IsValid())
			{
				ToolSettings->SaveConfig();
			}
		})
	];

	BrushCategory.AddCustomRow(LOCTEXT("BrushFalloffCategory", "Brush Falloff"), false)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("BrushFalloffLabel", "Falloff"))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.ToolTipText(LOCTEXT("BrushFalloffTooltip", "At 0, the brush has no falloff. At 1 it has exponential falloff."))
	]
	.ValueContent()
	[
		SNew(SSpinBox<float>)
		.MinValue(0.f)
		.MaxValue(1.f)
		.Value_Lambda([this]()
		{
			return ToolSettings.IsValid() ? ToolSettings->GetBrushConfig().Falloff : 1.f;
		})
		.OnValueChanged_Lambda([this](float NewValue)
		{
			if (ToolSettings.IsValid())
			{
				ToolSettings->BrushFalloffAmount = NewValue;
				ToolSettings->GetBrushConfig().Falloff = NewValue;
				FPropertyChangedEvent PropertyChangedEvent(UBrushBaseProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UBrushBaseProperties, BrushFalloffAmount)));
				ToolSettings->PostEditChangeProperty(PropertyChangedEvent);
			}
		})
		.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
		{
			if (ToolSettings.IsValid())
			{
				ToolSettings->SaveConfig();
			}
		})
	];

	BrushCategory.AddCustomRow(LOCTEXT("BrushAccumulateCategory", "Brush Accumulate"), false)
	.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
	{
		const bool bIsRelax = ToolSettings.IsValid() && ToolSettings->BrushMode == EWeightEditOperation::Relax;
		return bIsRelax ? EVisibility::Visible : EVisibility::Collapsed;
	}))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("BrushAccumulateLabel", "Accumulate"))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.ToolTipText(LOCTEXT("BrushAccumulateTooltip",
			"When enabled, repeated strokes over the same area continue to smooth. "
			"When disabled, each stroke produces a single bounded result."))
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked_Lambda([this]()
		{
			const bool bChecked = ToolSettings.IsValid() && ToolSettings->RelaxBrushAdvancedConfig.bAccumulate;
			return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
		{
			if (ToolSettings.IsValid())
			{
				ToolSettings->RelaxBrushAdvancedConfig.bAccumulate = (NewState == ECheckBoxState::Checked);
				ToolSettings->SaveConfig();
			}
		})
	];
}

void FSkinWeightDetailCustomization::AddSelectionUI(IDetailLayoutBuilder& DetailBuilder) const
{
	// custom display of selection editing tools
	IDetailCategoryBuilder& EditSelectionCategory = DetailBuilder.EditCategory("Edit Selection", FText::GetEmpty(), ECategoryPriority::Important);
	EditSelectionCategory.InitiallyCollapsed(true);

	// create a toolbar for the selection filter
	FSlimHorizontalToolBarBuilder ToolbarBuilder(MakeShared<FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(FModelingToolsEditorModeStyle::Get().Get(), "PolyEd.SelectionToolbar");
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	ToolbarBuilder.BeginSection("SelectionFilter");
	ToolbarBuilder.BeginBlockGroup();
	
	auto AddToggleButtonForBool = [&ToolbarBuilder, this](EComponentSelectionMode Mode, const FText& Label, const FText& Tooltip, const FName IconName)
	{
		ToolbarBuilder.AddToolBarButton(FUIAction(
		FExecuteAction::CreateLambda([this, Mode]()
		{
			if (ToolSettings.IsValid())
			{
				const bool bConvertSelection = FSlateApplication::Get().GetModifierKeys().IsControlDown();
				ToolSettings->SetComponentSelectionMode(Mode, bConvertSelection);
			}
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return ToolSettings.IsValid() ? ToolSettings->EditingMode == EWeightEditMode::Mesh : false;
		}),
		FIsActionChecked::CreateLambda([this, Mode]()
		{
			return ToolSettings.IsValid() ? ToolSettings->ComponentSelectionMode == Mode : false;
		})),
		NAME_None,	// Extension hook
		Label,		// Label
		Tooltip,	// Tooltip
		FSlateIcon(FModelingToolsEditorModeStyle::Get()->GetStyleSetName(), IconName),
		EUserInterfaceActionType::ToggleButton);
	};

	AddToggleButtonForBool(
		EComponentSelectionMode::Vertices,
		LOCTEXT("VerticesLabel", "Vertices"),
		LOCTEXT("VerticesTooltip", "Select mesh vertices. Ctrl-clicking converts the current selection to vertices."),
		"PolyEd.SelectCorners");
	AddToggleButtonForBool(
		EComponentSelectionMode::Edges,
		LOCTEXT("EdgesLabel", "Edges"),
		LOCTEXT("EdgesTooltip", "Select mesh edges. Ctrl-clicking converts the current selection to edges."),
		"PolyEd.SelectEdges");
	AddToggleButtonForBool(
		EComponentSelectionMode::Faces,
		LOCTEXT("FacesLabel", "Faces"),
		LOCTEXT("FacesTooltip", "Select mesh faces. Ctrl-clicking converts the current selection to faces."),
		"PolyEd.SelectFaces");

	ToolbarBuilder.EndBlockGroup();
	ToolbarBuilder.EndSection();

	// edit selection category
	EditSelectionCategory.AddCustomRow(LOCTEXT("EditSelectionRow", "Edit Selection"), false)
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.0f, WeightEditVerticalPadding)
			[
				SNew(SHorizontalBox)
				
				// Selection Modes
				+SHorizontalBox::Slot()
				[
					ToolbarBuilder.MakeWidget()
				]

				// Isolate selection
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.Style(FSkeletalMeshModelingToolsStyle::Get(), "SkeletalMeshModelingTools.ToggleButtonCheckbox")
					.HAlign(HAlign_Center)
					.ToolTipText(LOCTEXT("IsolateSelectedTooltip",
							"Shows only the selected faces in the viewport.\n"
							"Weight editing operations will not affect hidden vertices.\n"
							"NOTE: This only works on the target (main) mesh."))
					.IsEnabled_Lambda([this]()
					{
						if (Tool.IsValid())
						{
							// isolated selection only available on main mesh (for now)
						   const bool bHasSelection = Tool->GetMainMeshSelector()->IsAnyComponentSelected();
						   const bool bAlreadyIsolatingSelection = !Tool->GetSelectionIsolator()->CanIsolate();
						   return bHasSelection ||  bAlreadyIsolatingSelection;
						}
						return false;
					})
					.IsChecked_Lambda([this]()
					{
						if (Tool.IsValid())
						{
							const bool bAlreadyIsolatingSelection = !Tool->GetSelectionIsolator()->CanIsolate();
							return bAlreadyIsolatingSelection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
						return ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
					{
						if (UWeightToolSelectionIsolator* SelectionIsolator = Tool.IsValid() ? Tool->GetSelectionIsolator() : nullptr)
						{
							if (InCheckBoxState == ECheckBoxState::Checked)
							{
								SelectionIsolator->IsolateSelectionAsTransaction();
							}
							else
							{
								SelectionIsolator->UnIsolateSelectionAsTransaction();
							}
						}
					})
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							if (!Tool.IsValid() || !Tool->GetSelectionIsolator()->CanIsolate())
							{
								return LOCTEXT("ShowAllButtonLabel", "Show Full Mesh");
							}
								
							return LOCTEXT("IsolateButtonLabel", "Isolate Selected");
						})
					]
				]
			]
		]

		// GROW / SHRINK / FLOOD
		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
		
			+SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SButton)
				.IsEnabled_Lambda([this]
				{
					return Tool.IsValid() ? Tool->GetActiveMeshSelector()->IsAnyComponentSelected() : false;
				})
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("GrowSelectionButtonLabel", "Grow"))
				.ToolTipText(LOCTEXT("GrowSelectionTooltip",
						"Grow the current selection by adding connected neighbors to current selection.\n"))
				.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->GetActiveMeshSelector()->GrowSelection();
					}
					return FReply::Handled();
				})
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SButton)
				.IsEnabled_Lambda([this]
				{
					return Tool.IsValid() ? Tool->GetActiveMeshSelector()->IsAnyComponentSelected() : false;
				})
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("ShrinkSelectionButtonLabel", "Shrink"))
				.ToolTipText(LOCTEXT("ShrinkSelectionTooltip",
						"Shrink the current selection by removing components on the border of the current selection.\n"))
				.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->GetActiveMeshSelector()->ShrinkSelection();
					}
					return FReply::Handled();
				})
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SButton)
				.IsEnabled_Lambda([this]
				{
					return Tool.IsValid() ? Tool->GetActiveMeshSelector()->IsAnyComponentSelected() : false;
				})
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("FloodSelectionButtonLabel", "Flood"))
				.ToolTipText(LOCTEXT("FloodSelectionTooltip",
						"Flood the current selection by adding all connected components to the current selection.\n"))
				.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->GetActiveMeshSelector()->FloodSelection();
					}
					return FReply::Handled();
				})
			]
		]

		// SELECT AFFECTED VERTICES
		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
		
			+SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SButton)
				.IsEnabled_Lambda([this]
				{
					// only allow selecting affected vertices on the target/main mesh
					return ToolSettings.IsValid() ? ToolSettings->MeshSelectMode == EMeshTransferOption::Target : false;
				})
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("AffectedSelectionButtonLabel", "Affected"))
				.ToolTipText(LOCTEXT("AffectedSelectionTooltip",
						"Select vertices that are affected by the currently selected bone(s).\n"
						"Holding Shift or Ctrl will add or subtract affected vertices from the current selection."))
				.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->SelectAffected();
					}
					return FReply::Handled();
				})
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SButton)
				.IsEnabled_Lambda([this]
				{
					return Tool.IsValid() ? Tool->GetActiveMeshSelector()->IsAnyComponentSelected() : false;
				})
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("BorderSelectionButtonLabel", "Convert to Border"))
				.ToolTipText(LOCTEXT("BorderSelectionTooltip", "Select vertices on the border of the current selection."))
				.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->GetActiveMeshSelector()->SelectBorder();
					}
					return FReply::Handled();
				})
			]
		]

		// SELECT BY INFLUENCE COUNT
		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
		
			+SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SButton)
				.IsEnabled_Lambda([this]
				{
					// only allow selecting affected vertices on the target/main mesh
					return ToolSettings.IsValid() ? ToolSettings->MeshSelectMode == EMeshTransferOption::Target : false;
				})
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("InfluenceCountSelectionButtonLabel", "Influence Count"))
				.ToolTipText(LOCTEXT("InfluenceCountSelectionTooltip",
						"Select vertices that are affected by at least N influences."))
				.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid())
					{
						Tool->SelectByInfluenceCount(Tool->GetWeightToolProperties()->ClampSelectValue);
					}
					return FReply::Handled();
				})
			]

			+SHorizontalBox::Slot()
			.Padding(2.f, WeightEditVerticalPadding)
			[
				SNew(SSpinBox<int32>)
				.MinValue(1)
				.MaxValue(24)
				.Value_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->ClampSelectValue : 8;
				})
				.OnValueChanged_Lambda([this](int32 NewValue)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->ClampSelectValue = NewValue;
					}
				})
				.OnValueCommitted_Lambda([this](int32 NewValue, ETextCommit::Type CommitType)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->SaveConfig();
					}
				})
			]
		]
	];
	
	IDetailCategoryBuilder& EditWeightsCategory = DetailBuilder.EditCategory("Edit Weights", FText::GetEmpty(), ECategoryPriority::Important);
	EditWeightsCategory.InitiallyCollapsed(true);

	EditWeightsCategory.AddCustomRow(LOCTEXT("SelectMessageRow", "Select Vertices"), false)
	.WholeRowContent()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock)
		.Visibility_Lambda([this]()
		{
			return Tool.IsValid() && !Tool->HasActiveSelectionOnMainMesh() ? EVisibility::Visible : EVisibility::Collapsed;
		})
		.Text_Lambda([this]()
		{
			return LOCTEXT("NothingSelectedLabel", "Select vertices on target mesh to edit weights...");
		})
	];

	// FLOOD WEIGHTS SLIDER category
	ToolSettings->DirectEditState.Reset();
	EditWeightsCategory.AddCustomRow(LOCTEXT("FloodWeightsRow", "Flood Weights Slider"), false)
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled_Lambda([this]
		{
			return Tool.IsValid() && Tool->HasActiveSelectionOnMainMesh();
		})

		+SHorizontalBox::Slot()
		[
			SNew(SSegmentedControl<EWeightEditOperation>)
			.ToolTipText(LOCTEXT("InteractiveEditModeTooltip",
				"Add: applies the current weight plus the flood value to the new weight.\n"
				"Multiply: applies the current weight multiplied by the flood value to the new weight.\n"
				"This operation applies interactively while dragging the slider. It operates on the currently selected vertices."))
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->DirectEditState.EditMode : EWeightEditOperation::Add;
			})
			.OnValueChanged_Lambda([this](EWeightEditOperation Mode)
			{
				if(ToolSettings.IsValid())
				{
					ToolSettings->DirectEditState.EditMode = Mode;
					ToolSettings->DirectEditState.Reset();
				}
			})
			+SSegmentedControl<EWeightEditOperation>::Slot(EWeightEditOperation::Add)
			.Text(LOCTEXT("BrushAddMode", "Add"))
			+ SSegmentedControl<EWeightEditOperation>::Slot(EWeightEditOperation::Multiply)
			.Text(LOCTEXT("BrushMultiplyMode", "Multiply"))
		]
		
		+SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSpinBox<float>)
				.Visibility_Lambda([this]()
				{
					const bool bIsVisible = ToolSettings.IsValid() && ToolSettings->DirectEditState.EditMode != EWeightEditOperation::Relax;
					return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.MinSliderValue_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->DirectEditState.GetModeMinValue() : 0.f;
				})
				.MaxSliderValue_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->DirectEditState.GetModeMaxValue() : 0.f;
				})
				.MinValue_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->DirectEditState.GetModeMinValue() : 0.f;
				})
				.MaxValue_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->DirectEditState.GetModeMaxValue() : 0.f;
				})
				.Value_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->DirectEditState.CurrentValue : 0.f;
				})
				.OnValueChanged_Lambda([this](float NewValue)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->DirectEditState.CurrentValue = NewValue;
					
					   if (ToolSettings->DirectEditState.bInTransaction)
					   {
						   float Value = NewValue;
						   if (ToolSettings->DirectEditState.EditMode == EWeightEditOperation::Add)
						   {
							   Value = NewValue - ToolSettings->DirectEditState.StartValue;
						   }
						
						   if (Tool.IsValid())
						   {
						   		constexpr bool bShouldTransact = false;
					   			Tool->EditWeightsOnVertices(
									  Tool->GetCurrentBoneIndex(),
									  Value,
									  0, // iterations
									  ToolSettings->DirectEditState.EditMode,
									  Tool->GetMainMeshSelector()->GetSelectedVertices(),
									  bShouldTransact);
						   }
					   }
					}
				})
				.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
				{
					if (ToolSettings.IsValid())
					{
						if (!ToolSettings->DirectEditState.bInTransaction)
						{
							if (Tool.IsValid())
							{
								constexpr bool bShouldTransact = true;
								Tool->EditWeightsOnVertices(
									Tool->GetCurrentBoneIndex(),
									NewValue,
									0, // iterations
									ToolSettings->DirectEditState.EditMode,
									Tool->GetMainMeshSelector()->GetSelectedVertices(),
									bShouldTransact);
							}
						}
						ToolSettings->DirectEditState.bInTransaction = false;
					}
				})
				.OnBeginSliderMovement_Lambda([this]()
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->DirectEditState.StartValue = ToolSettings->DirectEditState.CurrentValue;
						ToolSettings->DirectEditState.bInTransaction = true;
					}
					if (Tool.IsValid())
					{
						Tool->BeginChange();
					}
				})
				.OnEndSliderMovement_Lambda([this](float)
				{
					const FText TransactionLabel = LOCTEXT("FloodWeightChange", "Flood weights on vertices.");
					if (Tool.IsValid())
					{
						Tool->EndChange(TransactionLabel);
					}

					if (ToolSettings.IsValid())
					{
						ToolSettings->DirectEditState.bInTransaction = false;

					   // reset multiply slider
					   if (ToolSettings->DirectEditState.EditMode == EWeightEditOperation::Multiply)
					   {
						   // multiplying operation is always relative to 1.0
						   ToolSettings->DirectEditState.CurrentValue = 1.0f;
						   ToolSettings->DirectEditState.StartValue = 1.0f;
					   }

					   // reset add slider
					   if (ToolSettings->DirectEditState.EditMode == EWeightEditOperation::Add)
					   {
						   // add operation is always relative to 0.0
						   ToolSettings->DirectEditState.CurrentValue = 0.0f;
						   ToolSettings->DirectEditState.StartValue = 0.0f;
					   }
					}
				})
				.ToolTipText(LOCTEXT("FloodWeightsToolTip", "Drag the slider to interactively adjust weights on the selected vertices."))
			]	
		]
	];

	// ADD WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("AddWeightsRow", "Add"), false)
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		.IsEnabled_Lambda([this]
		{
			return Tool.IsValid() ? Tool->HasActiveSelectionOnMainMesh() : false;
		})

		+ SVerticalBox::Slot()
		.Padding(WeightEditHorizontalPadding, WeightEditVerticalPadding)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SBox)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("AddWeightsButtonLabel", "Add"))
					.ToolTipText(LOCTEXT("AddButtonTooltip",
					"Add: applies the current weight plus the flood value to the new weight.\n"
					"This operation applies to the currently selected vertices."))
					.OnClicked_Lambda([this]()
					{
						if (Tool.IsValid() && ToolSettings.IsValid())
						{
							constexpr bool bShouldTransact = true;
							Tool->EditWeightsOnVertices(
								Tool->GetCurrentBoneIndex(),
								ToolSettings->AddStrength,
								0, // iterations 
								EWeightEditOperation::Add,
								Tool->GetMainMeshSelector()->GetSelectedVertices(),
								bShouldTransact);
						}
						return FReply::Handled();
					})
				]
			]

			+SHorizontalBox::Slot()
			[
				SNew(SSpinBox<float>)
				.MinValue(0)
				.MaxValue(1)
				.ToolTipText(LOCTEXT("AddWeightsSliderToolTip", "Adjust the value to Add to the selected vertices."))
				.Value_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->AddStrength : 1.f;
				})
				.OnValueChanged_Lambda([this](float NewValue)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->AddStrength = NewValue;
					}
				})
				.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->SaveConfig();
					}
				})
			]
		]
	];

	// REPLACE WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("ReplaceWeightsRow", "Replace"), false)
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		.IsEnabled_Lambda([this]
		{
			return Tool.IsValid() ? Tool->HasActiveSelectionOnMainMesh() : false;
		})

		+ SVerticalBox::Slot()
		.Padding(WeightEditHorizontalPadding, WeightEditVerticalPadding)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			[
				SNew(SBox)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("ReplaceWeightsButtonLabel", "Replace"))
					.ToolTipText(LOCTEXT("ReplaceButtonTooltip",
						"Replace: the weight of selected vertices is replaced by the specified value.\n"
						"This operation applies to the currently selected vertices."))
					.OnClicked_Lambda([this]()
					{
						if (Tool.IsValid() && ToolSettings.IsValid())
						{
							constexpr bool bShouldTransact = true;
							Tool->EditWeightsOnVertices(
								Tool->GetCurrentBoneIndex(),
								ToolSettings->ReplaceValue,
								0, // iterations
								EWeightEditOperation::Replace,
								Tool->GetMainMeshSelector()->GetSelectedVertices(),
								bShouldTransact);
						}
						return FReply::Handled();
					})
				]
			]

			+SHorizontalBox::Slot()
			[
				SNew(SSpinBox<float>)
				.MinValue(0.f)
				.MaxValue(1.f)
				.ToolTipText(LOCTEXT("ReplaceWeightsSliderToolTip", "Adjust the value to Replace on the selected vertices."))
				.Value_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->ReplaceValue : 1.f;
				})
				.OnValueChanged_Lambda([this](float NewValue)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->ReplaceValue = NewValue;
					}
				})
				.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->SaveConfig();
					}
				})
			]
		]
	];

	// AVERAGE WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("AverageWeightsRow", "Average"), false)
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		.IsEnabled_Lambda([this]{ return Tool->HasActiveSelectionOnMainMesh(); })

		+ SVerticalBox::Slot()
		.Padding(WeightEditHorizontalPadding, WeightEditVerticalPadding)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			[
				SNew(SBox)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("AverageWeightsButtonLabel", "Average"))
					.ToolTipText(LOCTEXT("AverageButtonTooltip",
						"Weights on all selected vertices are set to the average of all selected vertices."))
					.OnClicked_Lambda([this]()
					{
						if (ToolSettings.IsValid())
						{
							ToolSettings->WeightTool->AverageWeights(ToolSettings->AverageStrength);
						}
						return FReply::Handled();
					})
				]
			]

			+SHorizontalBox::Slot()
			[
				SNew(SSpinBox<float>)
				.MinValue(0.f)
				.MaxValue(1.f)
				.ToolTipText(LOCTEXT("AverageWeightsSliderToolTip", "Blend the amount to Average the weights on the selected vertices."))
				.Value_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->AverageStrength : 1.f;
				})
				.OnValueChanged_Lambda([this](float NewValue)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->AverageStrength = NewValue;
					}
				})
				.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->SaveConfig();
					}
				})
			]
		]
	];

	// RELAX WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("RelaxWeightsRow", "Relax"), false)
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		.IsEnabled_Lambda([this]
		{
			return Tool.IsValid() ? Tool->HasActiveSelectionOnMainMesh() : false;
		})

		+ SVerticalBox::Slot()
		.Padding(WeightEditHorizontalPadding, WeightEditVerticalPadding)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SBox)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("RelaxWeightsButtonLabel", "Relax"))
					.ToolTipText(LOCTEXT("RelaxButtonTooltip",
						"Relax: the weight of each selected vertex is replaced by the average of it's neighbors. \n"
						"This smooths weights across the mesh."))
					.OnClicked_Lambda([this]()
					{
						if (Tool.IsValid() && ToolSettings.IsValid())
						{
							constexpr bool bShouldTransact = true;
							constexpr int32 DefaultRelaxIterations = 5; // provides a reasonable falloff distance 
							Tool->EditWeightsOnVertices(
							   Tool->GetCurrentBoneIndex(),
							   ToolSettings->RelaxStrength,
							   DefaultRelaxIterations, 
							   EWeightEditOperation::Relax,
							   Tool->GetMainMeshSelector()->GetSelectedVertices(),
							   bShouldTransact);
						}
						return FReply::Handled();
					})
				]
			]

			+SHorizontalBox::Slot()
			[
				SNew(SSpinBox<float>)
				.MinValue(0)
				.MaxValue(1)
				.ToolTipText(LOCTEXT("RelaxWeightsSliderToolTip", "Blend the amount to Relax the weights on the selected vertices."))
				.Value_Lambda([this]()
				{
					return ToolSettings.IsValid() ? ToolSettings->RelaxStrength : 0.5f;
				})
				.OnValueChanged_Lambda([this](float NewValue)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->RelaxStrength = NewValue;
					}
				})
				.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->SaveConfig();
					}
				})
			]
		]
	];
	
	// MIRROR WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("MirrorWeightsRow", "Mirror"), false)
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		.IsEnabled_Lambda([this]{ return Tool->HasActiveSelectionOnMainMesh(); })
		
		+ SVerticalBox::Slot()
		.Padding(WeightEditHorizontalPadding, WeightEditVerticalPadding)
		[
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(WeightEditingLabelsPercent)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MirrorPlaneLabel", "Mirror Plane"))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.ToolTipText(LOCTEXT("MirrorPlaneTooltip", "The plane to copy weights across."))
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(SSegmentedControl<EAxis::Type>)
					.ToolTipText(LOCTEXT("MirrorAxisTooltip",
						"X: copies weights across the YZ plane.\n"
						"Y: copies weights across the XZ plane.\n"
						"Z: copies weights across the XY plane."))
					.Value_Lambda([this]()
					{
						return ToolSettings.IsValid() ? ToolSettings->MirrorAxis : TEnumAsByte<EAxis::Type>(EAxis::X);
					})
					.OnValueChanged_Lambda([this](EAxis::Type Mode)
					{
						if (ToolSettings.IsValid())
						{
							ToolSettings->MirrorAxis = Mode;
						}
					})
					+ SSegmentedControl<EAxis::Type>::Slot(EAxis::X)
					.Text(LOCTEXT("MirrorXLabel", "X"))
					+ SSegmentedControl<EAxis::Type>::Slot(EAxis::Y)
					.Text(LOCTEXT("MirrorYLabel", "Y"))
					+ SSegmentedControl<EAxis::Type>::Slot(EAxis::Z)
					.Text(LOCTEXT("MirrorZLabel", "Z"))
				]
					
				+SHorizontalBox::Slot()
				[
					SNew(SSegmentedControl<EMirrorDirection>)
					.ToolTipText(LOCTEXT("MirrorDirectionTooltip", "The direction that determines what side of the plane to copy weights from."))
					.Value_Lambda([this]()
					{
						return ToolSettings.IsValid() ? ToolSettings->MirrorDirection : EMirrorDirection::PositiveToNegative;
					})
					.OnValueChanged_Lambda([this](EMirrorDirection Mode)
					{
						if (ToolSettings.IsValid())
						{
							ToolSettings->MirrorDirection = Mode;
						}
					})
					+ SSegmentedControl<EMirrorDirection>::Slot(EMirrorDirection::PositiveToNegative)
					.Text(LOCTEXT("MirrorPosToNegLabel", "+ to -"))
					+ SSegmentedControl<EMirrorDirection>::Slot(EMirrorDirection::NegativeToPositive)
					.Text(LOCTEXT("MirrorNegToPosLabel", "- to +"))
				]
			]
		]

		+SVerticalBox::Slot()
		.Padding(0.f, WeightEditVerticalPadding)
		[
			SNew(SBox)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("MirrorWeightsButtonLabel", "Mirror"))
				.ToolTipText(LOCTEXT("MirrorButtonTooltip",
					"Weights are copied across the given plane in the given direction.\n"
					"This command operates on the selected vertices."))
				.OnClicked_Lambda([this]()
				{
					if (Tool.IsValid() && ToolSettings.IsValid())
					{
						Tool->MirrorWeights(ToolSettings->MirrorAxis, ToolSettings->MirrorDirection);
					}
					return FReply::Handled();
				})
			]
		]
	];

	// NORMALIZE WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("NormalizeWeightsRow", "Normalize"), false)
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled_Lambda([this]{ return Tool->HasActiveSelectionOnMainMesh(); })

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("NormalizeWeightsButtonLabel", "Normalize"))
			.ToolTipText(LOCTEXT("NormalizeWeightsTooltip",
					"Forces the weights on the selected vertices to sum to 1.\n"
					"This command operates on the selected vertices."))
			.IsEnabled_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->EditingMode == EWeightEditMode::Mesh : false;
			})
			.OnClicked_Lambda([this]()
			{
				if (Tool.IsValid())
				{
					Tool->NormalizeWeights();
				}
				return FReply::Handled();
			})
		]

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("HammerWeightsButtonLabel", "Hammer"))
			.ToolTipText(LOCTEXT("HammerWeightsTooltip",
					"Copies the weight of the nearest non-selected vertex.\n"
					"This command operates on the selected vertices."))
			.IsEnabled_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->EditingMode == EWeightEditMode::Mesh : false;
			})
			.OnClicked_Lambda([this]()
			{
				if (Tool.IsValid())
				{
					Tool->HammerWeights();
				}
				return FReply::Handled();
			})
		]
	];

	// PRUNE WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("PruneWeightsRow", "Prune"), false)
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled_Lambda([this]{ return Tool->HasActiveSelectionOnMainMesh(); })

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("PruneWeightsButtonLabel", "Prune"))
			.ToolTipText(LOCTEXT("PruneButtonTooltip",
				"Removes influences with weights below the given threshold value.\n"
				"Pruned bones are removed from the list of bones affecting the given vertex.\n"
				"Pruned bones will no longer receive weight when a vertex is normalized.\n"
				"This command operates on the selected vertices."))
			.OnClicked_Lambda([this]()
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->WeightTool->PruneWeights(ToolSettings->PruneValue, TArray<BoneIndex>());
				}
				return FReply::Handled();
			})
		]

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SSpinBox<float>)
			.MinValue(0.f)
			.MaxValue(1.f)
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->PruneValue : 0.01;
			})
			.OnValueChanged_Lambda([this](float NewValue)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->PruneValue = NewValue;
				}
			})
			.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->SaveConfig();
				}
			})
		]
	];

	// CLAMP INFLUENCES category
	EditWeightsCategory.AddCustomRow(LOCTEXT("ClampInfluencesRow", "Clamp"), false)
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled_Lambda([this]
		{
			return Tool.IsValid() ? Tool->HasActiveSelectionOnMainMesh() : false;
		})

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("ClampInfluencesButtonLabel", "Clamp"))
			.ToolTipText(LOCTEXT("ClampInfluencesButtonTooltip",
				"Clamp the number of influences to not exceed the target value.\n"
				"Removes smallest influences first.\n"
				"This command operates on the selected vertices."))
			.OnClicked_Lambda([this]()
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->WeightTool->ClampInfluences(ToolSettings->ClampValue);
				}
				return FReply::Handled();
			})
		]

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SSpinBox<int32>)
			.MinValue(1)
			.MaxValue(24)
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->ClampValue : 8;
			})
			.OnValueChanged_Lambda([this](int32 NewValue)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->ClampValue = NewValue;
				}
			})
			.OnValueCommitted_Lambda([this](int32 NewValue, ETextCommit::Type CommitType)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->SaveConfig();
				}
			})
		]
	];

	// COPY/PASTE WEIGHTS category
	EditWeightsCategory.AddCustomRow(LOCTEXT("CopyPasteWeightsRow", "Copy Paste"), false)
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled_Lambda([this]
		{
			return Tool.IsValid() ? Tool->HasActiveSelectionOnMainMesh() : false;
		})

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SBox)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("CopyWeightsButtonLabel", "Copy"))
				.ToolTipText(LOCTEXT("CopyButtonTooltip",
					"Copy the average weights of the selected vertices to the clipboard. \n"
					"This is designed to work with the Paste command."))
				.OnClicked_Lambda([this]()
				{
					if (ToolSettings.IsValid())
					{
						ToolSettings->WeightTool->CopyWeights();
					}
					return FReply::Handled();
				})
			]
		]

		+SHorizontalBox::Slot()
		.Padding(2.f, WeightEditVerticalPadding)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("PasteWeightsButtonLabel", "Paste"))
			.ToolTipText(LOCTEXT("PasteButtonTooltip",
				"Paste the weights on the selected vertices.\n"
				"This command requires the clipboard contain weights from the Copy command."))
			.OnClicked_Lambda([this]()
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->WeightTool->PasteWeights();
				}
				return FReply::Handled();
			})
		]
	];
	
	// VERTEX EDITOR category
	EditWeightsCategory.AddCustomRow(LOCTEXT("VertexEditorRow", "Component Editor"), false)
	.WholeRowContent()
	[
		SNew(SBox)
		.IsEnabled_Lambda([this]
		{
			return Tool.IsValid() ? Tool->HasActiveSelectionOnMainMesh() : false;
		})
		[
			SNew(SVertexWeightEditor, ToolSettings->WeightTool)
		]
	];
}

void FSkinWeightDetailCustomization::AddTransferUI(IDetailLayoutBuilder& DetailBuilder) const
{
	if (!ensure(Tool.IsValid()))
	{
		return;
	}
	
	IDetailCategoryBuilder& TransferWeightsCategory = DetailBuilder.EditCategory("WeightTransfer", FText::GetEmpty(), ECategoryPriority::Important);
	TransferWeightsCategory.InitiallyCollapsed(true);

	// TRANSFER BUTTON
	TransferWeightsCategory.AddCustomRow(LOCTEXT("TransferWeightsRow", "Transfer Weights"), false)
	.WholeRowContent()
	[
		SNew(SBox)
		.Padding(0.f, WeightEditVerticalPadding)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("TransferWeightsButtonLabel", "Transfer Weights"))
			.ToolTipText(LOCTEXT("TransferButtonTooltip",
						"Weights are transferred from the source skeletal mesh.\n"
						"Vertices may be selected on the source and/or target mesh to filter which parts to copy from and which parts to copy to.\n"
						"If either mesh has no vertices selected, the whole mesh is considered.\n"))
			.OnClicked_Lambda([this]()
			{
				if (Tool.IsValid())
				{
					Tool->GetWeightTransferManager()->TransferWeights();
				}
				return FReply::Handled();
			})
			.IsEnabled_Lambda([this]()
			{
				return Tool.IsValid() ? Tool->GetWeightTransferManager()->CanTransferWeights() : false;
			})
		]
	];

	// SKELETAL MESH ASSET INPUT
	const TSharedRef<IPropertyHandle> SourceSkeletalMeshHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, SourceSkeletalMesh), USkinWeightsPaintToolProperties::StaticClass());
	TransferWeightsCategory.AddProperty(SourceSkeletalMeshHandle);
	
	// LOD
	const TSharedRef<IPropertyHandle> LODHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, SourceLOD), USkinWeightsPaintToolProperties::StaticClass());
	TransferWeightsCategory.AddProperty(LODHandle);

	// PROFILE
	const TSharedRef<IPropertyHandle> SourceProfileHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, SourceSkinWeightProfile), USkinWeightsPaintToolProperties::StaticClass());
	TransferWeightsCategory.AddProperty(SourceProfileHandle);

	// MESH SELECTION OPTION (SOURCE OR TARGET)
	const TSharedRef<IPropertyHandle> TransferSelectModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, MeshSelectMode), USkinWeightsPaintToolProperties::StaticClass());
	FDetailWidgetRow& TransferSelectModeRow = TransferWeightsCategory.AddCustomRow(LOCTEXT("SelectionModeRow", "Selection Mode"), false)
	.NameContent()
	[
		TransferSelectModeHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SBox)
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SSegmentedControl<EMeshTransferOption>)
			.ToolTipText(LOCTEXT("SelectionSourceTooltip",
					"Choose which mesh to select components on (vertices/edges/faces).\n"
					"Weights will be transferred from selected components on the source to selected components on the target.\n"
					"If no components are selected on either the source or target, the whole mesh will be considered.\n"
					"Source: The mesh to copy weights FROM.\n"
					"Target: The mesh to copy weights TO (the main mesh in the tool)."))
			.Value_Lambda([this]()
			{
				return ToolSettings.IsValid() ? ToolSettings->MeshSelectMode : EMeshTransferOption::Target;
			})
			.OnValueChanged_Lambda([this](EMeshTransferOption Mode)
			{
				if (ToolSettings.IsValid())
				{
					ToolSettings->MeshSelectMode = Mode;
					ToolSettings->WeightTool->UpdateSelectorState();
				}
			})
			+ SSegmentedControl<EMeshTransferOption>::Slot(EMeshTransferOption::Source)
			.Text(LOCTEXT("SourceMode", "Source"))
			+SSegmentedControl<EMeshTransferOption>::Slot(EMeshTransferOption::Target)
			.Text(LOCTEXT("TargetMode", "Target"))
		]
	];

	// PREVIEW OFFSET
	const TSharedRef<IPropertyHandle> PreviewOffsetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightsPaintToolProperties, SourcePreviewOffset), USkinWeightsPaintToolProperties::StaticClass());
	TransferWeightsCategory.AddProperty(PreviewOffsetHandle);
}

void SVertexWeightItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
	Element = InArgs._Element;
	ParentTable = InArgs._ParentTable;
	SMultiColumnTableRow<TSharedPtr<FWeightEditorElement>>::Construct(FSuperRowType::FArguments(), OwnerTableView);
}

TSharedRef<SWidget> SVertexWeightItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ColumnName_Bone)
	{
		const FName BoneName = ParentTable->Tool.IsValid() ? ParentTable->Tool->GetBoneNameFromIndex(Element->BoneIndex) : NAME_None;
		return SNew(STextBlock)
			.Text_Lambda([Tool = ParentTable->Tool, BoneName, BoneIndex = Element->BoneIndex]()
				{
					bool bLocked = Tool.IsValid() ? Tool->GetFinalLockedBoneIndices().Contains(BoneIndex) : false;
					return FText::Format(LOCTEXT("VertexWeightBoneNameFormat", "{0}{1}"),
						FText::FromName(BoneName),
						bLocked ? LOCTEXT("VertexWeightBoneNameLockedText", " (Locked)") : FText());
				});
	}

	if (ColumnName == ColumnName_Weight)
	{
		return SNew(SNumericEntryBox<float>)
		.AllowSpin(true)
		.MinSliderValue(0.f)
		.MinValue(0.f)
		.MaxSliderValue(1.0f)
		.MaxValue(1.f)
		.Value_Lambda([this]()
		{
			if (bInTransaction)
			{
				return ValueDuringSlide;
			}

			if (!ParentTable->Tool.IsValid())
			{
				return 0.f;
			}
			
			return ParentTable->Tool->GetAverageWeightOnBone(Element->BoneIndex, ParentTable->Tool->GetMainMeshSelector()->GetSelectedVertices());
		})
		.OnValueChanged_Lambda([this](float NewValue)
		{
			if (bInTransaction)
			{
				ValueDuringSlide = NewValue;

				if (ParentTable->Tool.IsValid())
				{
					const bool bScalingUp = NewValue >= ValueAtStartOfSlide || FMath::IsNearlyEqual(ValueAtStartOfSlide, 0.f);
				   const float RangeEnd = bScalingUp ? 1.f : 0.f;
				   float RelativeScale = (NewValue - ValueAtStartOfSlide) * 1.f / (RangeEnd - ValueAtStartOfSlide);
				   RelativeScale *= bScalingUp ? 1.f : -1.f;
				   constexpr bool bShouldTransact = false;
				   ParentTable->Tool->EditWeightsOnVertices(
					   Element->BoneIndex,
					   RelativeScale,
					   0, // iterations
					   EWeightEditOperation::RelativeScale,
					   ParentTable->Tool->GetMainMeshSelector()->GetSelectedVertices(),
					   bShouldTransact);
				}
			}
		})
		.OnValueCommitted_Lambda([this](float NewValue, ETextCommit::Type CommitType)
		{
			if (!bInTransaction)
			{
				if (ParentTable->Tool.IsValid())
				{
					constexpr bool bShouldTransact = true;
					ParentTable->Tool->EditWeightsOnVertices(
					   Element->BoneIndex,
					   NewValue,
					   0, // iterations,
					   EWeightEditOperation::Replace,
					   ParentTable->Tool->GetMainMeshSelector()->GetSelectedVertices(),
					   bShouldTransact);
				}
			}
			bInTransaction = false;
		})
		.OnBeginSliderMovement_Lambda([this]()
		{
			if (ParentTable->Tool.IsValid())
			{
				ParentTable->Tool->BeginChange();
				ValueAtStartOfSlide = ParentTable->Tool->GetAverageWeightOnBone(Element->BoneIndex, ParentTable->Tool->GetMainMeshSelector()->GetSelectedVertices());
			}
			ValueDuringSlide = ValueAtStartOfSlide;
			bInTransaction = true;
		})
		.OnEndSliderMovement_Lambda([this](float)
		{
			if (ParentTable->Tool.IsValid())
			{
				const FText TransactionLabel = LOCTEXT("DirectWeightChange", "Scale weights on vertices.");
				ParentTable->Tool->EndChange(TransactionLabel);
			}
			bInTransaction = false;
		})
		.ToolTipText(LOCTEXT("WeightSliderToolTip", "Set the weight on this bone for the selected vertices."));
	}

	if (ColumnName == ColumnName_Prune)
	{
		return SNew(SHorizontalBox)
		
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.HAlign(HAlign_Right)
			.ToolTipText(LOCTEXT("PruneInfluence", "Prune the influence from the selected vertices."))
			.OnClicked_Lambda([this]() -> FReply
			{
				// use a negative threshold weight that no weight will ever
				// be below because we only want to prune based on the bone, regardless of the weight
				if (ParentTable->Tool.IsValid())
				{
					constexpr float NegativeThreshold = -1.0f;
					ParentTable->Tool->PruneWeights(NegativeThreshold, {Element->BoneIndex});
				}
				return FReply::Handled();
			})
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	checkNoEntry();
	return SNullWidget::NullWidget;
}

SVertexWeightEditor::~SVertexWeightEditor()
{
	if (Tool.IsValid())
	{
		Tool->OnSelectionChanged.RemoveAll(this);
		Tool->OnWeightsChanged.RemoveAll(this);
		Tool.Reset();
	}
}

void SVertexWeightEditor::Construct(const FArguments& InArgs, USkinWeightsPaintTool* InSkinTool)
{
	Tool = InSkinTool;

	ChildSlot
	[
		SNew(SBox)
		[
			SAssignNew( ListView, SWeightEditorListViewType )
			.Visibility_Lambda([this]()
			{
				return Tool.IsValid() && !Tool->GetMainMeshSelector()->GetSelectedVertices().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.SelectionMode(ESelectionMode::Single)
			.OnSelectionChanged(this, &SVertexWeightEditor::OnSelectionChanged)
			.ListItemsSource( &ListViewItems )
			.OnGenerateRow_Lambda([this](TSharedPtr<FWeightEditorElement> Element, const TSharedRef<STableViewBase>& OwnerTableView)
			{
				return SNew(SVertexWeightItem, OwnerTableView).Element(Element).ParentTable(SharedThis(this));
			})
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(ColumnName_Bone)
				.HAlignHeader(HAlign_Center)
				.DefaultLabel(LOCTEXT("WeightEditorBoneColumn", "Bone"))
				+ SHeaderRow::Column(ColumnName_Weight)
				.HAlignHeader(HAlign_Center)
				.DefaultLabel(LOCTEXT("WeightEditorWeightColumn", "Weight (Average)"))
				+ SHeaderRow::Column(ColumnName_Prune)
				.HAlignHeader(HAlign_Center)
				.DefaultLabel(LOCTEXT("WeightEditorPruneColumn", "Prune"))
				.FixedWidth(60.f)
			)
		]
	];

	RefreshView();
	
	Tool->OnSelectionChanged.AddSP(this, &SVertexWeightEditor::RefreshView);
	Tool->OnWeightsChanged.AddSP(this, &SVertexWeightEditor::RefreshView);
}

void SVertexWeightEditor::RefreshView()
{
	if (!Tool.IsValid())
	{
		return; 
	}
	
	// get all bones affecting the selected vertices
	TArray<int32> Influences;
	Tool->GetInfluences(Tool->GetMainMeshSelector()->GetSelectedVertices(), Influences);

	// Preserve existing items's widget if possible to avoid their spinbox's destruction triggering false value commits.
	TArray<TSharedPtr<FWeightEditorElement>> PreviousItems = MoveTemp(ListViewItems);
	ListViewItems.Reset();
	ListViewItems.Reserve(Influences.Num());
	for (const int32 InfluenceIndex : Influences)
	{
		const TSharedPtr<FWeightEditorElement>* ExistingItem = PreviousItems.FindByPredicate(
			[InfluenceIndex](const TSharedPtr<FWeightEditorElement>& Item)
			{
				return Item.IsValid() && Item->BoneIndex == InfluenceIndex;
			});
		ListViewItems.Add(ExistingItem ? *ExistingItem : MakeShareable(new FWeightEditorElement(InfluenceIndex)));
	}
	
	ListView->RequestListRefresh();
}

void SVertexWeightEditor::OnSelectionChanged(TSharedPtr<FWeightEditorElement> Selection, ESelectInfo::Type SelectInfo)
{
	if (!Selection.IsValid() ||
		!Tool.IsValid())
	{
		return;
	}
	const USkinWeightsPaintTool& ToolRef = *Tool.Get();
	
	if (const TSharedPtr<SReferenceSkeletonTree>& SkeletonTree = ToolRef.GetToolSkeletonTree())
	{
		const FName BoneName = ToolRef.GetBoneNameFromIndex(Selection->BoneIndex);

		constexpr bool bFrameSelection = true;
		SkeletonTree->SelectItemFromNames({ BoneName }, bFrameSelection);
	}
}

#undef LOCTEXT_NAMESPACE
