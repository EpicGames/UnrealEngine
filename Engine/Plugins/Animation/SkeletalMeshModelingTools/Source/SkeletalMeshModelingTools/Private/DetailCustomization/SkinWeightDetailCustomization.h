// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

class USkinWeightsPaintTool;
class USkinWeightsPaintToolProperties;

class FSkinWeightDetailCustomization : public IDetailCustomization
{
public:

	virtual ~FSkinWeightDetailCustomization() override;

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FSkinWeightDetailCustomization);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	void OnSelectionChanged()
	{
		ToolSettings.Get()->DirectEditState.Reset();
	};

private:

	void AddBrushUI(IDetailLayoutBuilder& DetailBuilder) const;
	void AddSelectionUI(IDetailLayoutBuilder& DetailBuilder) const;
	void AddTransferUI(IDetailLayoutBuilder& DetailBuilder) const;

	IDetailLayoutBuilder* CurrentDetailBuilder = nullptr;
	TWeakObjectPtr<USkinWeightsPaintToolProperties> ToolSettings;
	TWeakObjectPtr<USkinWeightsPaintTool> Tool;

	static float WeightSliderWidths;
	static float WeightEditingLabelsPercent;
	static float WeightEditVerticalPadding;
	static float WeightEditHorizontalPadding;
};

struct FWeightEditorElement
{
	int32 BoneIndex;

	FWeightEditorElement(int32 InBoneIndex) : BoneIndex(InBoneIndex) {}
};

class SVertexWeightEditor;
class SVertexWeightItem : public SMultiColumnTableRow<TSharedPtr<FWeightEditorElement>>
{
public:

	SLATE_BEGIN_ARGS(SVertexWeightItem) {}
	SLATE_ARGUMENT(TSharedPtr<FWeightEditorElement>, Element)
	SLATE_ARGUMENT(TSharedPtr<SVertexWeightEditor>, ParentTable)
	SLATE_END_ARGS()

	virtual ~SVertexWeightItem() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<FWeightEditorElement> Element;
	TSharedPtr<SVertexWeightEditor> ParentTable;
	bool bInTransaction = false;
	float ValueAtStartOfSlide;
	float ValueDuringSlide;
};

typedef SListView< TSharedPtr<FWeightEditorElement> > SWeightEditorListViewType;

class SVertexWeightEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVertexWeightEditor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, USkinWeightsPaintTool* InSkinTool);

	virtual ~SVertexWeightEditor() override;
	
	void RefreshView();

private:
	/** Called when the selection in this editor changed */
	void OnSelectionChanged(TSharedPtr<FWeightEditorElement> Selection, ESelectInfo::Type SelectInfo);

	// the vertex weight list view
	TSharedPtr<SWeightEditorListViewType> ListView;
	TArray< TSharedPtr<FWeightEditorElement> > ListViewItems;
	TWeakObjectPtr<USkinWeightsPaintTool> Tool;

	friend SVertexWeightItem;
};
