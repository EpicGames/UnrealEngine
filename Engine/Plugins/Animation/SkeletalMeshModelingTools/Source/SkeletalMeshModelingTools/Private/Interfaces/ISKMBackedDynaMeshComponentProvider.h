// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "ISKMBackedDynaMeshComponentProvider.generated.h"

class USkeletalMeshBackedDynamicMeshComponent;

UINTERFACE(MinimalAPI)
class USkeletalMeshBackedDynamicMeshComponentProvider :
	public UInterface
{
	GENERATED_BODY()
};


class ISkeletalMeshBackedDynamicMeshComponentProvider
{
	GENERATED_BODY()

public:
	virtual USkeletalMeshBackedDynamicMeshComponent* GetComponent(UObject* SourceObject) = 0;
};
