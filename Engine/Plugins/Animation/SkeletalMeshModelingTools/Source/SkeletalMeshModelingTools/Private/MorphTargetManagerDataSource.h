// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "MorphTargetManagerDataSource.generated.h"

UINTERFACE(MinimalAPI)
class UMorphTargetManagerDataSource : public UInterface
{
	GENERATED_BODY()
};

class IMorphTargetManagerDataSource
{
	GENERATED_BODY()

public:
	virtual TArray<FName> GetMorphTargets() = 0;

	virtual float GetMorphTargetWeight(FName MorphTarget) = 0;
	virtual void  SetMorphTargetWeight(FName MorphTarget, float Weight) = 0;

	virtual bool  GetMorphTargetAutoFill(FName MorphTarget) = 0;
	virtual void  SetMorphTargetAutoFill(FName MorphTarget, bool bAutoFill, float PreviousOverrideWeight) = 0;

	virtual FName GetEditingMorphTarget() const = 0;
	virtual void  SetEditingMorphTarget(FName MorphTarget) = 0;

	virtual FName         AddMorphTarget(FName InName) = 0;
	virtual TArray<FName> AddMorphTargetsIfMissing(const TArray<FName>& Names) = 0;
	virtual FName         RenameMorphTarget(FName OldName, FName NewName) = 0;
	virtual void          RemoveMorphTargets(const TArray<FName>& Names) = 0;
	virtual TArray<FName> DuplicateMorphTargets(const TArray<FName>& Names) = 0;
	virtual void          MirrorMorphTargets(const TArray<FName>& Names) = 0;
	virtual void          FlipMorphTargets(const TArray<FName>& Names) = 0;
	virtual FName         MergeMorphTargets(const TArray<FName>& Names) = 0;
	virtual void          ApplyCurrentWeightToMorphTarget(FName Name) = 0;
	virtual void          GenerateFlippedMorphTargets(const TArray<TPair<FName, FName>>& InPairs) = 0;

	virtual FSimpleMulticastDelegate& OnMorphTargetDataChanged() = 0;
};
