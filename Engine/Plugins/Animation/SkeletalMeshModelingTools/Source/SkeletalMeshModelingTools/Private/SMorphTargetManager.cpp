// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMorphTargetManager.h"

#include "SkeletalMeshEditingCache.h"
#include "SkeletalMeshModelingToolsCommands.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Animation/MorphTarget.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor/EditorEngine.h"
#include "Internationalization/Regex.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

extern UNREALED_API UEditorEngine* GEditor;


#define LOCTEXT_NAMESPACE "SMorphTargetList"

namespace MorphTargetManagerLocal
{
	struct FMorphTargetInfo
	{
		FName Name = NAME_None;
		float Weight = 1.0f;
		bool bIsRenaming = false;
		TSharedPtr<SInlineEditableTextBlock> EditableText;
	};
	static const FName ColumnId_MorphTargetNameLabel( "MorphTargetName" );
	static const FName ColumnID_MorphTargetWeightLabel( "Weight" );
}

class SMorphTargetManagerListRow : public SMultiColumnTableRow< FMorphTargetInfoPtr >
{
public:

	SLATE_BEGIN_ARGS( SMorphTargetManagerListRow ) {}

	/** The item for this row **/
	SLATE_ARGUMENT( FMorphTargetInfoPtr, Item )

		/* The SMorphTargetViewer that we push the morph target weights into */
		SLATE_ARGUMENT( SMorphTargetManager* , Manager)

		/* Widget used to display the list of morph targets */
		SLATE_ARGUMENT( SMorphTargetManagerListType* , ListView )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView )
	{
		Item = InArgs._Item;
		Manager = InArgs._Manager;
		ListView = InArgs._ListView;
		
		SMultiColumnTableRow< FMorphTargetInfoPtr >::Construct( FSuperRowType::FArguments(), OwnerTableView );
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		using namespace MorphTargetManagerLocal;
		
		if ( ColumnName == ColumnId_MorphTargetNameLabel )
		{
			FText MorphNameText = FText::FromName(Item->Name);
			return
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 4.0f , 4.0f, 0.0f, 4.0f )
				.VAlign( VAlign_Center )
				[
					SNew(SCheckBox)
					.ToolTipText(LOCTEXT("MorphTargetEditToggleToolTip", "Activate Morph Target for editing"))
					.Style(FAppStyle::Get(), "RadioButton")
					.OnCheckStateChanged(this, &SMorphTargetManagerListRow::OnSetEditingMorphTarget)
					.IsChecked(this, &SMorphTargetManagerListRow::IsEditingMorphTarget)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding( 0.0f, 4.0f )
				.VAlign( VAlign_Center )
				[
					SAssignNew(Item->EditableText, SInlineEditableTextBlock)
					.Text_Lambda([this]()
					{
						return FText::FromName(Item->Name);
					})
					.HighlightText(Manager,  &SMorphTargetManager::GetHighlightText, MorphNameText)
					.OnTextCommitted(this, &SMorphTargetManagerListRow::OnRenameMorphTarget)
					.IsSelected(this, &SMorphTargetManagerListRow::IsSelected)
					.IsReadOnly_Lambda([this]() { return !Item->bIsRenaming; })
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 4.0f, 4.0f)
				.VAlign( VAlign_Center )
				[
					SNew(STextBlock)
					.Text( LOCTEXT("EditingMorphTargetMarker", "(Editing)") )
					.Visibility_Lambda([this]()
					{
						return IsEditingMorphTarget() == ECheckBoxState::Checked ? EVisibility::Visible : EVisibility::Collapsed;
					})
				];
		}
		else if ( ColumnName == ColumnID_MorphTargetWeightLabel )
		{
			// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
			return
				SNew( SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding( 0.0f, 4.0f )
				.VAlign( VAlign_Center )
				[
					SNew( SSpinBox<float> )
					.MinSliderValue(0.f)
					.MaxSliderValue(1.f)
					.Value( this, &SMorphTargetManagerListRow::GetWeight )
					.OnBeginSliderMovement(this, &SMorphTargetManagerListRow::OnBeginSlideMorphTargetWeight)
					.OnEndSliderMovement(this, &SMorphTargetManagerListRow::OnEndSlideMorphTargetWeight)
					.OnValueChanged( this, &SMorphTargetManagerListRow::OnMorphTargetWeightChanged )
					.OnValueCommitted( this, &SMorphTargetManagerListRow::OnMorphTargetWeightValueCommitted )
					.IsEnabled(this, &SMorphTargetManagerListRow::IsMorphTargetWeightSliderEnabled)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 0.0f, 4.0f )
				.VAlign( VAlign_Center )
				[
					SNew(SCheckBox)
					.ToolTipText(LOCTEXT("MorphTargetAutoFillToolTip", "When checked, animation system takes control of morph target weights"))
					.OnCheckStateChanged(this, &SMorphTargetManagerListRow::OnMorphTargetAutoFillChecked)
					.IsChecked(this, &SMorphTargetManagerListRow::IsMorphTargetAutoFillChangedChecked)
				];
		}
		

		return SNullWidget::NullWidget;
	}

	

private:

	/**
	* Called when the user begins/ends dragging the slider on the SSpinBox
	*/
	void OnBeginSlideMorphTargetWeight()
	{
		GEditor->BeginTransaction(LOCTEXT("OverrideMorphTargetWeight", "Override Morph Target Weight"));
	};
	void OnEndSlideMorphTargetWeight(float Value)
	{
		GEditor->EndTransaction();
	};
	
	/**
	* Called when the user changes the value of the SSpinBox
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnMorphTargetWeightChanged( float NewWeight )
	{
		Item->Weight = NewWeight;
		Manager->SetMorphTargetWeight(Item->Name, NewWeight);
	};
	
	/**
	* Called when the user types the value and enters
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnMorphTargetWeightValueCommitted( float NewWeight, ETextCommit::Type CommitType)
	{
		OnMorphTargetWeightChanged( NewWeight );
	};
	
	/**
	* Called to know if we enable or disable the weight sliders
	*/
	bool IsMorphTargetWeightSliderEnabled() const
	{
		return true;
	};
	
	/** Auto fill check call back functions */
	void OnMorphTargetAutoFillChecked(ECheckBoxState InState)
	{
		Manager->SetMorphTargetAutoFill(Item->Name, InState == ECheckBoxState::Checked, Item->Weight);
	};
	ECheckBoxState IsMorphTargetAutoFillChangedChecked() const
	{
		return Manager->GetMorphTargetAutoFill(Item->Name) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	};

	/** Auto fill check call back functions */
	void OnSetEditingMorphTarget(ECheckBoxState InState)
	{
		FName EditingMorphTarget = InState == ECheckBoxState::Checked ? Item->Name : NAME_None;
		Manager->SetEditingMorphTarget(EditingMorphTarget);
	};
	ECheckBoxState IsEditingMorphTarget() const
	{
		return Manager->IsEditingMorphTarget(Item->Name) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	};
	
	/**
	* Returns the weight of this morph target
	*
	* @return SearchText - The new number the SSpinBox is set to
	*
	*/
	float GetWeight() const
	{
		return Manager->GetMorphTargetWeight(Item->Name);
	};

	void OnRenameMorphTarget(const FText& InNewName, ETextCommit::Type)
	{
		Item->bIsRenaming = false;
		Item->Name = Manager->RenameMorphTarget( Item->Name, *InNewName.ToString());
	}

	/* The SMorphTargetViewer that we push the morph target weights into */
	SMorphTargetManager* Manager = nullptr;

	/** Widget used to display the list of morph targets */
	SMorphTargetManagerListType* ListView = nullptr;

	/** The name and weight of the morph target */
	FMorphTargetInfoPtr Item;
};



void SMorphTargetManager::Construct(const FArguments& InArgs)
{
	using namespace MorphTargetManagerLocal;

	WeakDataSource = InArgs._DataSource;

	BindCommands();
	
	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.f, 2.f))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.f, 0.0))
			[
				SNew(SPositiveActionButton)
				.OnGetMenuContent( this, &SMorphTargetManager::CreateNewMenuWidget )
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 6.f, 0.f))
			[
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButtonWithIcon"))
				.ForegroundColor(FSlateColor::UseStyle())
				.OnGetMenuContent(this, &SMorphTargetManager::CreateFilterOptionsMenuWidget)
				.ToolTipText(LOCTEXT("MorphTargetFilterOptionsToolTip", "Filter options"))
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SLayeredImage,
						TAttribute<const FSlateBrush*>::CreateLambda([this]() -> const FSlateBrush*
						{
							return (bInvertFilter || bDisableFilter) ? FAppStyle::Get().GetBrush("Icons.BadgeModified") : nullptr;
						}),
						TAttribute<FSlateColor>())
					.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			// Filter entry
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SMorphTargetManager::OnFilterTextChanged )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.f, 0.f))
			[
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.OnGetMenuContent(this, &SMorphTargetManager::CreateSortMenuWidget)
				.ToolTipText(LOCTEXT("MorphTargetSortToolTip", "Change how morph targets are sorted"))
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.SortDown"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.f, 0.0))
			[
				// A dummy widget to inform user the morph target viewer was replaced with morph target manager
				// and how they can access the default viewer
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked_Lambda([](){return ECheckBoxState::Checked;})
				[
					SNew(STextBlock)
					.ToolTipText(LOCTEXT("UsingMorphTargetManagerToolTip", "Deactivate Skeletal Mesh Editing Tools to access default Morph Target Viewer"))
					.Text(LOCTEXT("UsingMorphTargetManagerNote", "Editing Tools Enabled"))
				]
			]
		]

		+ SVerticalBox::Slot()
		.Padding( FMargin( 0.0f, 2.0f, 0.0f, 0.0f ) )
		.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
		[
			SAssignNew( ListView , SMorphTargetManagerListType )
			.ListItemsSource( &List)
			.OnGenerateRow(this, &SMorphTargetManager::GenerateMorphTargetRow )
			.OnMouseButtonDoubleClick(this, &SMorphTargetManager::OnDoubleClickMorphTarget)
			.OnContextMenuOpening( this, &SMorphTargetManager::OnGetContextMenuContent )
			.OnItemScrolledIntoView(this, &SMorphTargetManager::OnItemScrolledIntoView)
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_MorphTargetNameLabel )
				.DefaultLabel( LOCTEXT( "MorphTargetNameLabel", "Morph Target Name" ) )

				+ SHeaderRow::Column( ColumnID_MorphTargetWeightLabel )
				.DefaultLabel( LOCTEXT( "MorphTargetWeightLabel", "Weight" ) )
			)
		]	
	];

	if (IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get())
	{
		InvalidationHandle = DataSource->OnMorphTargetDataChanged().AddSP(this, &SMorphTargetManager::RefreshList);
	}

	RefreshList();
}

SMorphTargetManager::~SMorphTargetManager()
{
	if (IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get())
	{
		if (InvalidationHandle.IsValid())
		{
			DataSource->OnMorphTargetDataChanged().Remove(InvalidationHandle);
		}
	}
}

FReply SMorphTargetManager::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SMorphTargetManager::BindCommands()
{
	const FSkeletalMeshModelingToolsCommands& Commands = FSkeletalMeshModelingToolsCommands::Get();
	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	CommandList = MakeShared<FUICommandList>(); 
	CommandList->MapAction(Commands.NewMorphTarget, FSimpleDelegate::CreateSP(this, &SMorphTargetManager::AddMorphTarget) );
	CommandList->MapAction(Commands.AddMissingMorphTargetsFromSkeletalMesh, FExecuteAction::CreateSP(this, &SMorphTargetManager::OpenAddMissingMorphTargetsDialog));
	CommandList->MapAction(
		GenericCommands.Rename,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::RenameSelectedMorphTarget), 
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanRename) );
	CommandList->MapAction(
		GenericCommands.Delete,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::RemoveSelectedMorphTargets), 
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanRemove) );
	
	CommandList->MapAction(
		GenericCommands.Duplicate,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::DuplicateSelectedMorphTargets), 
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanDuplicate));
	
	CommandList->MapAction(
		Commands.MirrorSelectedMorphTarget,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::MirrorSelectedMorphTargets),
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanMirror));

	CommandList->MapAction(
		Commands.FlipSelectedMorphTarget,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::FlipSelectedMorphTargets),
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanFlip));

	CommandList->MapAction(
		Commands.MergeSelectedMorphTargets,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::MergeSelectedMorphTargets),
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanMerge));

	CommandList->MapAction(
		Commands.ApplyCurrentWeightToMorphTarget,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::ApplyCurrentWeightToSelectedMorphTarget),
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanApplyCurrentWeight));

	CommandList->MapAction(
		Commands.ToggleSelectedMorphTargetWeight,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::ToggleSelectedMorphTargetsWeight),
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanToggleSelectedMorphTargetsWeight));

	CommandList->MapAction(
		Commands.ConfigureMorphTargetNamingConvention,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::OpenConfigureNamingConventionDialog));

	CommandList->MapAction(
		Commands.GenerateFlippedMorphTargets,
		FExecuteAction::CreateSP(this, &SMorphTargetManager::GenerateFlippedMorphTargetsForSelection),
		FCanExecuteAction::CreateSP(this, &SMorphTargetManager::CanGenerateFlippedMorphTargets));
}

void SMorphTargetManager::RefreshList()
{
	using namespace MorphTargetManagerLocal;
	
	List.Reset();

	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		ListView->RequestListRefresh();
		return;
	}

	FText FilterText = GetFilterText();

	const TArray<FName> MorphTargets = DataSource->GetMorphTargets();

	for (const FName& MorphTarget : MorphTargets)
	{
		bool bAddToList = true;
		if (!bDisableFilter && !FilterText.IsEmpty())
		{
			FRegexPattern Pattern(FilterText.ToString(), ERegexPatternFlags::CaseInsensitive);
			FRegexMatcher Matcher(Pattern, MorphTarget.ToString());

			const bool bMatches = Matcher.FindNext();
			bAddToList = bInvertFilter ? !bMatches : bMatches;
		}

		if (bAddToList)
		{
			TSharedPtr<FMorphTargetInfo> Info = MakeShared<FMorphTargetInfo>();
			Info->Name = MorphTarget;
			Info->Weight = DataSource->GetMorphTargetWeight(MorphTarget);
			List.Add(Info);
		}
	}

	if (SortMode == ESortMode::Alphabetical)
	{
		List.Sort([](const FMorphTargetInfoPtr& A, const FMorphTargetInfoPtr& B)
		{
			return A->Name.Compare(B->Name) < 0;
		});
	}

	ListView->RequestListRefresh();

	if (!Pending.ToSelect.IsEmpty())
	{
		SelectMorphTargets(Pending.ToSelect.Array());
		Pending.ToSelect.Reset();
	}

	if (!Pending.ToScrollTo.IsNone())
	{
		if (const FMorphTargetInfoPtr* Info = List.FindByPredicate(
				[&](const FMorphTargetInfoPtr& Item){ return Item->Name == Pending.ToScrollTo; }))
		{
			ListView->RequestScrollIntoView(*Info);
		}
		Pending.ToScrollTo = NAME_None;
	}

	// Pending.ToRename is drained by OnItemScrolledIntoView, but drop it here if the target
	// no longer exists (e.g. undo rolled back the merge) so the request doesn't dangle.
	if (!Pending.ToRename.IsNone() && !List.ContainsByPredicate(
			[&](const FMorphTargetInfoPtr& Item){ return Item->Name == Pending.ToRename; }))
	{
		Pending.ToRename = NAME_None;
	}
}

void SMorphTargetManager::SelectMorphTargets(const TArray<FName>& MorphTargets)
{
	ListView->ClearSelection();
	
	FMorphTargetInfoPtr First = nullptr; 
	for (const FName& Name : MorphTargets)
	{
		FMorphTargetInfoPtr* Info = List.FindByPredicate([&](const FMorphTargetInfoPtr& InInfo)
		{
			return InInfo->Name == Name;
		});

		if (Info)
		{
			ListView->SetItemSelection(*Info, true);
			if (!First)
			{
				First = *Info;
			}
		}
	}
	
	ListView->RequestNavigateToItem(First);
}

TSharedRef<SWidget> SMorphTargetManager::CreateNewMenuWidget()
{
	const FSkeletalMeshModelingToolsCommands& Commands = FSkeletalMeshModelingToolsCommands::Get();

	static constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, CommandList);
	
	MenuBuilder.BeginSection("NewMorphTarget", LOCTEXT("AddNewMorphTargetOperations", "Morph Targets"));
	MenuBuilder.AddMenuEntry(Commands.NewMorphTarget);
	MenuBuilder.AddMenuEntry(Commands.AddMissingMorphTargetsFromSkeletalMesh);
	MenuBuilder.EndSection();
		
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SMorphTargetManager::CreateSortMenuWidget()
{
	static constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	MenuBuilder.BeginSection("MorphTargetSort", LOCTEXT("MorphTargetSort", "Sort By"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MorphTargetSortCustom", "Custom"),
		{},
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMorphTargetManager::SetSortMode, ESortMode::Custom),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMorphTargetManager::IsSortMode, ESortMode::Custom)),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MorphTargetSortAlphabetical", "Alphabetical"),
		LOCTEXT("MorphTargetSortAlphabeticalToolTip", "Sort by morph target name"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMorphTargetManager::SetSortMode, ESortMode::Alphabetical),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMorphTargetManager::IsSortMode, ESortMode::Alphabetical)),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SMorphTargetManager::SetSortMode(ESortMode NewMode)
{
	if (SortMode == NewMode)
	{
		return;
	}
	SortMode = NewMode;
	RefreshList();
}

TSharedRef<SWidget> SMorphTargetManager::CreateFilterOptionsMenuWidget()
{
	static constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	MenuBuilder.BeginSection("MorphTargetFilterOptions", LOCTEXT("MorphTargetFilterOptions", "Filter"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MorphTargetInvertFilter", "Invert Filtering"),
		LOCTEXT("MorphTargetInvertFilterToolTip", "When enabled, hide morph targets that match the filter text instead of showing them"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMorphTargetManager::ToggleInvertFilter),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMorphTargetManager::IsInvertFilterEnabled)),
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MorphTargetDisableFilter", "Disable Filtering"),
		LOCTEXT("MorphTargetDisableFilterToolTip", "When enabled, ignore the filter text and show every morph target. Lets you keep the text in place while peeking at the full list."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SMorphTargetManager::ToggleDisableFilter),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SMorphTargetManager::IsDisableFilterEnabled)),
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SMorphTargetManager::ToggleInvertFilter()
{
	bInvertFilter = !bInvertFilter;
	RefreshList();
}

void SMorphTargetManager::ToggleDisableFilter()
{
	bDisableFilter = !bDisableFilter;
	RefreshList();
}

void SMorphTargetManager::OnFilterTextChanged(const FText& Text)
{
	// View-only change (filter text) — refresh directly.
	// Data mutations go through IMorphTargetManagerDataSource::OnMorphTargetDataChanged instead.
	RefreshList();
}

FText SMorphTargetManager::GetFilterText() const
{
	return NameFilterBox->GetText(); 
}

FText SMorphTargetManager::GetHighlightText(FText InName) const
{
	FText FilterText = GetFilterText();
	if (!bDisableFilter && !FilterText.IsEmpty())
	{
		FRegexPattern Pattern(FilterText.ToString(), ERegexPatternFlags::CaseInsensitive);
		FString Name = InName.ToString();
		FRegexMatcher Matcher(Pattern, Name);
		if (Matcher.FindNext())
		{
			const int Begin = Matcher.GetMatchBeginning();	
			const int End = Matcher.GetMatchEnding();
			const int MatchSize = End - Begin;
			FString Highlight = Name.Mid(Begin, MatchSize);
			return FText::FromString(Highlight);
		}
	}

	return {};
}

TSharedPtr<SWidget> SMorphTargetManager::OnGetContextMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("MorphTargetManagerAction", LOCTEXT( "MorphsAction", "Selected Item Actions" ) );

	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
	MenuBuilder.AddMenuEntry(FSkeletalMeshModelingToolsCommands::Get().MirrorSelectedMorphTarget);
	MenuBuilder.AddMenuEntry(FSkeletalMeshModelingToolsCommands::Get().FlipSelectedMorphTarget);
	MenuBuilder.AddMenuEntry(FSkeletalMeshModelingToolsCommands::Get().MergeSelectedMorphTargets);
	MenuBuilder.AddMenuEntry(FSkeletalMeshModelingToolsCommands::Get().ApplyCurrentWeightToMorphTarget);
	MenuBuilder.AddMenuEntry(FSkeletalMeshModelingToolsCommands::Get().ToggleSelectedMorphTargetWeight);

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("MorphTargetAdvanced", LOCTEXT("MorphsAdvanced", "Advanced"));

	MenuBuilder.AddMenuEntry(FSkeletalMeshModelingToolsCommands::Get().ConfigureMorphTargetNamingConvention);
	MenuBuilder.AddMenuEntry(FSkeletalMeshModelingToolsCommands::Get().GenerateFlippedMorphTargets);

	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

void SMorphTargetManager::OnItemScrolledIntoView(FMorphTargetInfoPtr InItem, const TSharedPtr<ITableRow>& InTableRow)
{
	if (Pending.ToRename.IsNone() || !InItem.IsValid() || InItem->Name != Pending.ToRename)
	{
		return;
	}
	Pending.ToRename = NAME_None;

	if (InItem->EditableText.IsValid())
	{
		InItem->bIsRenaming = true;
		InItem->EditableText->EnterEditingMode();
	}
}

void SMorphTargetManager::SetMorphTargetWeight(FName MorphTarget, float Weight)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("OverrideMorphTargetWeight", "Override Morph Target Weight"));
	DataSource->SetMorphTargetWeight(MorphTarget, Weight);
}

float SMorphTargetManager::GetMorphTargetWeight(FName MorphTarget)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return 0.0f;
	}
	return DataSource->GetMorphTargetWeight(MorphTarget);
}

void SMorphTargetManager::SetMorphTargetAutoFill(FName MorphTarget, bool bAutoFill, float PreviousOverrideWeight)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}
	FText Title = bAutoFill ? LOCTEXT("ClearOverride", "Clear Morph Target Override") : LOCTEXT("OverrideMorphTargetWeight", "Override Morph Target Weight");
	FScopedTransaction Transaction(Title);
	DataSource->SetMorphTargetAutoFill(MorphTarget, bAutoFill, PreviousOverrideWeight);
}

bool SMorphTargetManager::GetMorphTargetAutoFill(FName MorphTarget)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return false;
	}
	return DataSource->GetMorphTargetAutoFill(MorphTarget);
}

void SMorphTargetManager::SetEditingMorphTarget(FName MorphTarget)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}
	FText Title = MorphTarget != NAME_None ? LOCTEXT("EnabledMorphTargetEditing", "Enable Morph Target for editing") : LOCTEXT("DisabledMorphTargetEditing", "Disabled Morph Target For Editing");
	FScopedTransaction Transaction(Title);
	DataSource->SetEditingMorphTarget(MorphTarget);
}

bool SMorphTargetManager::IsEditingMorphTarget(FName MorphTarget)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return false;
	}
	return DataSource->GetEditingMorphTarget() == MorphTarget;
}

void SMorphTargetManager::AddMorphTarget()
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("AddMorphTarget", "Add Morph Target"));
	const FName NewMorphTarget = DataSource->AddMorphTarget(TEXT("NewMorphTarget"));
	Pending.ToSelect   = { NewMorphTarget };
	Pending.ToScrollTo = NewMorphTarget;
	Pending.ToRename   = NewMorphTarget;
}

void SMorphTargetManager::OpenAddMissingMorphTargetsDialog()
{
	TStrongObjectPtr<UMorphTargetManagerAddMissingMorphsSetting> Settings(NewObject<UMorphTargetManagerAddMissingMorphsSetting>());

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("AddMissingMorphTargetsTitle", "Add Empty Morph Targets From Template"))
		.ClientSize(FVector2D(440.f, 200.f))
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;
	const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(Settings.Get());

	bool bConfirmed = false;
	TWeakPtr<SWindow> WeakWindow = Window;
	Window->SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(4.f)
		[
			DetailsView
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(3.f)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			+ SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
				.Text(LOCTEXT("AddMissingOK", "OK"))
				.OnClicked_Lambda([WeakWindow, &bConfirmed]()
				{
					bConfirmed = true;
					if (TSharedPtr<SWindow> Pinned = WeakWindow.Pin())
					{
						Pinned->RequestDestroyWindow();
					}
					return FReply::Handled();
				})
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("AddMissingCancel", "Cancel"))
				.OnClicked_Lambda([WeakWindow]()
				{
					if (TSharedPtr<SWindow> Pinned = WeakWindow.Pin())
					{
						Pinned->RequestDestroyWindow();
					}
					return FReply::Handled();
				})
			]
		]);

	FSlateApplication::Get().AddModalWindow(Window, AsShared());

	if (!bConfirmed)
	{
		return;
	}
	USkeletalMesh* Source = Settings->SourceSkeletalMesh;
	if (!Source)
	{
		return;
	}

	AddMissingMorphTargetsFromSkeletalMesh(Source);
}

void SMorphTargetManager::AddMissingMorphTargetsFromSkeletalMesh(USkeletalMesh* SourceMesh)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource || !SourceMesh)
	{
		return;
	}

	const TArray<TObjectPtr<UMorphTarget>>& SourceMorphs = SourceMesh->GetMorphTargets();
	TArray<FName> SourceNames;
	SourceNames.Reserve(SourceMorphs.Num());
	for (const TObjectPtr<UMorphTarget>& Morph : SourceMorphs)
	{
		if (Morph)
		{
			SourceNames.Add(Morph->GetFName());
		}
	}
	if (SourceNames.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddMissingMorphTargets", "Add Empty Morph Targets From Template"));
	const TArray<FName> Created = DataSource->AddMorphTargetsIfMissing(SourceNames);
	if (Created.IsEmpty())
	{
		return;
	}

	Pending.ToSelect = TSet<FName>(Created);
	Pending.ToScrollTo = Created[0];
}

void SMorphTargetManager::OnDoubleClickMorphTarget(FMorphTargetInfoPtr Item)
{
	if (!Item.IsValid())
	{
		return;
	}

	const float CurrentWeight = GetMorphTargetWeight(Item->Name);
	const float NewWeight = (CurrentWeight != 1.0f) ? 1.0f : 0.0f;
	SetMorphTargetWeight(Item->Name, NewWeight);
}

FName SMorphTargetManager::RenameMorphTarget(FName InOldName, FName InNewName)
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return InOldName;
	}
	FScopedTransaction Transaction(LOCTEXT("RenameMorphTarget", "Rename Morph Target"));
	FName NewName = DataSource->RenameMorphTarget(InOldName, InNewName);
	Pending.ToSelect   = { NewName };
	
	return NewName;
}

void SMorphTargetManager::RenameSelectedMorphTarget()
{
	TArray<FMorphTargetInfoPtr> Items = ListView->GetSelectedItems();
	Items[0]->bIsRenaming = true;
	Items[0]->EditableText->EnterEditingMode();
}

bool SMorphTargetManager::CanRename()
{
	return ListView->GetSelectedItems().Num() == 1;
}

void SMorphTargetManager::RemoveSelectedMorphTargets()
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}

	TArray<FMorphTargetInfoPtr> Items = ListView->GetSelectedItems();

	TArray<FName> Names;
	for (const FMorphTargetInfoPtr& Item : Items)
	{
		Names.Add(Item->Name);
	}

	FText Title = Names.Num() > 1 ? LOCTEXT("RemoveMorph", "Remove Morph Targets") : LOCTEXT("RemoveMorphs", "Remove Morph Targets");
	FScopedTransaction Transaction(Title);
	DataSource->RemoveMorphTargets(Names);
}

bool SMorphTargetManager::CanRemove()
{
	return ListView->GetSelectedItems().Num() > 0;
}

void SMorphTargetManager::DuplicateSelectedMorphTargets()
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}

	TArray<FMorphTargetInfoPtr> Items = ListView->GetSelectedItems();

	TArray<FName> Names;
	for (const FMorphTargetInfoPtr& Item : Items)
	{
		Names.Add(Item->Name);
	}

	FText Title = Names.Num() > 1 ? LOCTEXT("DuplicateMorphTargets", "Duplicate Morph Targets") : LOCTEXT("DuplicateMorphTarget", "Duplicate Morph Target");
	FScopedTransaction Transaction(Title);
	const TArray<FName> Duplicated = DataSource->DuplicateMorphTargets(Names);
	Pending.ToSelect = TSet<FName>(Duplicated);
	if (!Duplicated.IsEmpty())
	{
		Pending.ToScrollTo = Duplicated[0];
	}
}

bool SMorphTargetManager::CanDuplicate()
{
	return ListView->GetSelectedItems().Num() > 0;
}

void SMorphTargetManager::MirrorSelectedMorphTargets()
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}

	TArray<FMorphTargetInfoPtr> Items = ListView->GetSelectedItems();
	if (Items.IsEmpty())
	{
		return;
	}

	TArray<FName> Names;
	Names.Reserve(Items.Num());
	for (const FMorphTargetInfoPtr& Item : Items)
	{
		Names.Add(Item->Name);
	}

	FText Title = Names.Num() > 1
		? LOCTEXT("MirrorMorphTargetsTransaction", "Mirror Morph Targets")
		: FText::Format(LOCTEXT("MirrorMorphTransaction", "Mirror {0}"), FText::FromName(Names[0]));
	FScopedTransaction Transaction(Title);

	DataSource->MirrorMorphTargets(Names);
}

bool SMorphTargetManager::CanMirror()
{
	return ListView->GetSelectedItems().Num() >= 1;
}

void SMorphTargetManager::FlipSelectedMorphTargets()
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}

	TArray<FMorphTargetInfoPtr> Items = ListView->GetSelectedItems();
	if (Items.IsEmpty())
	{
		return;
	}

	TArray<FName> Names;
	Names.Reserve(Items.Num());
	for (const FMorphTargetInfoPtr& Item : Items)
	{
		Names.Add(Item->Name);
	}

	FText Title = Names.Num() > 1
		? LOCTEXT("FlipMorphTargetsTransaction", "Flip Morph Targets")
		: FText::Format(LOCTEXT("FlipMorphTransaction", "Flip {0}"), FText::FromName(Names[0]));
	FScopedTransaction Transaction(Title);

	DataSource->FlipMorphTargets(Names);
}

bool SMorphTargetManager::CanFlip()
{
	return ListView->GetSelectedItems().Num() >= 1;
}

void SMorphTargetManager::MergeSelectedMorphTargets()
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}

	TArray<FMorphTargetInfoPtr> Items = ListView->GetSelectedItems();
	if (Items.Num() < 2)
	{
		return;
	}

	TArray<FName> Names;
	Names.Reserve(Items.Num());
	for (const FMorphTargetInfoPtr& Item : Items)
	{
		Names.Add(Item->Name);
	}

	FScopedTransaction Transaction(LOCTEXT("MergeMorphTargets", "Merge Morph Targets"));
	const FName MergedName = DataSource->MergeMorphTargets(Names);
	if (MergedName.IsNone())
	{
		return;
	}

	Pending.ToSelect   = { MergedName };
	Pending.ToScrollTo = MergedName;
	Pending.ToRename   = MergedName;
}

bool SMorphTargetManager::CanMerge()
{
	return ListView->GetSelectedItems().Num() >= 2;
}

void SMorphTargetManager::ApplyCurrentWeightToSelectedMorphTarget()
{
	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}

	TArray<FMorphTargetInfoPtr> Items = ListView->GetSelectedItems();
	check(Items.Num() == 1);

	FName MorphTargetName = Items[0]->Name;
	FText Title = FText::Format(LOCTEXT("ApplyCurrentWeightMorphTransaction", "Apply Current Weight to {0}"), FText::FromName(MorphTargetName));
	FScopedTransaction Transaction(Title);

	DataSource->ApplyCurrentWeightToMorphTarget(MorphTargetName);
}

bool SMorphTargetManager::CanApplyCurrentWeight()
{
	return ListView->GetSelectedItems().Num() == 1;
}

void SMorphTargetManager::ToggleSelectedMorphTargetsWeight()
{
	const TArray<FMorphTargetInfoPtr> Selected = ListView->GetSelectedItems();
	for (const FMorphTargetInfoPtr& Item : Selected)
	{
		if (Item.IsValid())
		{
			const float CurrentWeight = GetMorphTargetWeight(Item->Name);
			const float NewWeight = (CurrentWeight != 1.0f) ? 1.0f : 0.0f;
			SetMorphTargetWeight(Item->Name, NewWeight);
		}
	}
}

bool SMorphTargetManager::CanToggleSelectedMorphTargetsWeight()
{
	return ListView->GetSelectedItems().Num() > 0;
}

namespace MorphTargetManagerLocal
{
	// Splits a wildcard pattern at the single '*'. Returns false if the pattern does not contain
	// exactly one '*'. e.g. "*_l" -> Prefix="", Suffix="_l"; "L_*" -> Prefix="L_", Suffix="".
	static bool SplitPattern(const FString& Pattern, FString& OutPrefix, FString& OutSuffix)
	{
		int32 First = INDEX_NONE;
		int32 Last = INDEX_NONE;
		Pattern.FindChar(TEXT('*'), First);
		Pattern.FindLastChar(TEXT('*'), Last);
		if (First == INDEX_NONE || First != Last)
		{
			return false;
		}
		OutPrefix = Pattern.Left(First);
		OutSuffix = Pattern.Mid(First + 1);
		return true;
	}

	// Returns true and writes the wildcard-matched stem into OutStem if Name matches "Prefix*Suffix".
	// NOTE: FString::StartsWith/EndsWith return false for empty prefix/suffix, so we have to short-circuit
	// those checks ourselves — the empty string is trivially a prefix/suffix of any name. Without this guard
	// the default "*_l"/"*_r" patterns (empty prefix) would never match anything.
	static bool ExtractStem(const FString& Name, const FString& Prefix, const FString& Suffix, FString& OutStem)
	{
		if (Name.Len() < Prefix.Len() + Suffix.Len())
		{
			return false;
		}
		if (!Prefix.IsEmpty() && !Name.StartsWith(Prefix, ESearchCase::CaseSensitive))
		{
			return false;
		}
		if (!Suffix.IsEmpty() && !Name.EndsWith(Suffix, ESearchCase::CaseSensitive))
		{
			return false;
		}
		OutStem = Name.Mid(Prefix.Len(), Name.Len() - Prefix.Len() - Suffix.Len());
		return true;
	}
}

void SMorphTargetManager::OpenConfigureNamingConventionDialog()
{
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("ConfigureNamingConventionTitle", "Configure Naming Convention"))
		.ClientSize(FVector2D(420.f, 180.f))
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;
	const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(GetMutableDefault<UMorphTargetNamingConventionSettings>());

	TWeakPtr<SWindow> WeakWindow = Window;
	Window->SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(4.f)
		[
			DetailsView
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(8.f)
		[
			SNew(SButton)
			.Text(LOCTEXT("CloseConfigureNamingConvention", "Close"))
			.OnClicked_Lambda([WeakWindow]()
			{
				if (TSharedPtr<SWindow> Pinned = WeakWindow.Pin())
				{
					Pinned->RequestDestroyWindow();
				}
				return FReply::Handled();
			})
		]);

	FSlateApplication::Get().AddModalWindow(Window, AsShared());
}

void SMorphTargetManager::GenerateFlippedMorphTargetsForSelection()
{
	using namespace MorphTargetManagerLocal;

	IMorphTargetManagerDataSource* DataSource = WeakDataSource.Get();
	if (!DataSource)
	{
		return;
	}

	const UMorphTargetNamingConventionSettings* Settings = GetDefault<UMorphTargetNamingConventionSettings>();
	FString LeftPrefix, LeftSuffix, RightPrefix, RightSuffix;
	if (!SplitPattern(Settings->LeftPattern, LeftPrefix, LeftSuffix))
	{
		return;
	}
	if (!SplitPattern(Settings->RightPattern, RightPrefix, RightSuffix))
	{
		return;
	}

	TArray<TPair<FName, FName>> Pairs;
	for (const FMorphTargetInfoPtr& Item : ListView->GetSelectedItems())
	{
		FString Stem;
		if (!ExtractStem(Item->Name.ToString(), LeftPrefix, LeftSuffix, Stem))
		{
			continue;
		}
		const FName RightName(*(RightPrefix + Stem + RightSuffix));
		Pairs.Add({Item->Name, RightName});
	}

	if (Pairs.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("GenerateFlippedMorphTargets", "Generate Flipped Morph Targets"));
	DataSource->GenerateFlippedMorphTargets(Pairs);

	TSet<FName> RightNames;
	RightNames.Reserve(Pairs.Num());
	for (const TPair<FName, FName>& Pair : Pairs)
	{
		RightNames.Add(Pair.Value);
	}
	Pending.ToSelect = MoveTemp(RightNames);
	Pending.ToScrollTo = Pairs[0].Value;
}

bool SMorphTargetManager::CanGenerateFlippedMorphTargets()
{
	using namespace MorphTargetManagerLocal;

	const UMorphTargetNamingConventionSettings* Settings = GetDefault<UMorphTargetNamingConventionSettings>();
	FString LeftPrefix, LeftSuffix;
	if (!SplitPattern(Settings->LeftPattern, LeftPrefix, LeftSuffix))
	{
		return false;
	}

	FString IgnoredRightPrefix, IgnoredRightSuffix;
	if (!SplitPattern(Settings->RightPattern, IgnoredRightPrefix, IgnoredRightSuffix))
	{
		return false;
	}

	for (const FMorphTargetInfoPtr& Item : ListView->GetSelectedItems())
	{
		FString Stem;
		if (ExtractStem(Item->Name.ToString(), LeftPrefix, LeftSuffix, Stem))
		{
			return true;
		}
	}
	return false;
}


TSharedRef<class ITableRow> SMorphTargetManager::GenerateMorphTargetRow(
	FMorphTargetInfoPtr MorphTargetItem,
	const TSharedRef<STableViewBase>& TableViewBase)
{
	TSharedPtr<SMorphTargetManagerListRow> Row =
		SNew(SMorphTargetManagerListRow, TableViewBase)
		.Item( MorphTargetItem )
		.Manager(this)
		.ListView(ListView.Get());
	
	return Row.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
