// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MorphTargetManagerDataSource.h"
#include "SkeletalMeshNotifier.h"
#include "UObject/Object.h"
#include "UObject/WeakInterfacePtr.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

#include "SMorphTargetManager.generated.h"

namespace MorphTargetManagerLocal
{
	struct FMorphTargetInfo;
}
typedef TSharedPtr<MorphTargetManagerLocal::FMorphTargetInfo> FMorphTargetInfoPtr;
typedef SListView< FMorphTargetInfoPtr > SMorphTargetManagerListType;


class SSearchBox;
class USkeletalMesh;
class UMorphTargetModifier;
class SMorphTargetManager;
class FUICommandList;


/** Transient settings displayed in the "Add Empty Morph Targets From Template" modal dialog. */
UCLASS(Transient)
class UMorphTargetManagerAddMissingMorphsSetting : public UObject
{
	GENERATED_BODY()

public:
	/** Template Skeletal Mesh to read Morph Target names from. Only the names are read — no delta data is transferred. */
	UPROPERTY(EditAnywhere, Category = "Template", meta = (DisplayName = "Template Skeletal Mesh"))
	TObjectPtr<USkeletalMesh> SourceSkeletalMesh = nullptr;
};



class SMorphTargetManager : public SCompoundWidget
{
public:
	/** How rows in the morph target list are ordered. */
	enum class ESortMode : uint8
	{
		/** Preserve the order returned by the data source. */
		Custom,
		/** Sort by morph target name. */
		Alphabetical
	};

	SLATE_BEGIN_ARGS( SMorphTargetManager ) {}
		SLATE_ARGUMENT(TWeakInterfacePtr<IMorphTargetManagerDataSource>, DataSource)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	/**
	* Destructor - resets the morph targets
	*
	*/
	virtual ~SMorphTargetManager();

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	void BindCommands();
	void RefreshList();
	void SelectMorphTargets(const TArray<FName>& MorphTargets);
	
	TSharedRef<SWidget> CreateNewMenuWidget();
	TSharedRef<SWidget> CreateSortMenuWidget();
	void SetSortMode(ESortMode NewMode);
	bool IsSortMode(ESortMode Mode) const { return SortMode == Mode; }
	TSharedRef<SWidget> CreateFilterOptionsMenuWidget();
	void ToggleInvertFilter();
	bool IsInvertFilterEnabled() const { return bInvertFilter; }
	void ToggleDisableFilter();
	bool IsDisableFilterEnabled() const { return bDisableFilter; }
	void OnFilterTextChanged(const FText& Text);
	FText GetFilterText() const;
	FText GetHighlightText(FText InName) const;
	
	TSharedRef<class ITableRow> GenerateMorphTargetRow(FMorphTargetInfoPtr MorphTargetItem, const TSharedRef<STableViewBase>& TableViewBase);
	TSharedPtr<SWidget> OnGetContextMenuContent();
	void OnItemScrolledIntoView(FMorphTargetInfoPtr InItem, const TSharedPtr<ITableRow>& InTableRow);

	void SetMorphTargetWeight(FName MorphTarget, float Weight);
	float GetMorphTargetWeight(FName MorphTarget);
	void SetMorphTargetAutoFill(FName MorphTarget, bool bAutoFill, float PreviousOverrideWeight);
	bool GetMorphTargetAutoFill(FName MorphTarget);
	void SetEditingMorphTarget(FName MorphTarget);
	bool IsEditingMorphTarget(FName MorphTarget);

	void AddMorphTarget();
	void OpenAddMissingMorphTargetsDialog();
	void AddMissingMorphTargetsFromSkeletalMesh(USkeletalMesh* SourceMesh);
	FName RenameMorphTarget(FName InOldName, FName InNewName);
	void OnDoubleClickMorphTarget(FMorphTargetInfoPtr Item);
protected:
	void RenameSelectedMorphTarget();
	bool CanRename();
	void RemoveSelectedMorphTargets();
	bool CanRemove();
	void DuplicateSelectedMorphTargets();
	bool CanDuplicate();
	void MirrorSelectedMorphTargets();
	bool CanMirror();
	void FlipSelectedMorphTargets();
	bool CanFlip();
	void MergeSelectedMorphTargets();
	bool CanMerge();
	void ApplyCurrentWeightToSelectedMorphTarget();
	bool CanApplyCurrentWeight();
	void ToggleSelectedMorphTargetsWeight();
	bool CanToggleSelectedMorphTargetsWeight();
	void OpenConfigureNamingConventionDialog();
	void GenerateFlippedMorphTargetsForSelection();
	bool CanGenerateFlippedMorphTargets();

	TSharedPtr<SSearchBox> NameFilterBox;
	TSharedPtr<SMorphTargetManagerListType> ListView;

	TArray<FMorphTargetInfoPtr> List;
	TArray<FMorphTargetInfoPtr> FullList;

	TWeakObjectPtr<UMorphTargetModifier> Modifier;

	struct FPendingRowActions
	{
		TSet<FName> ToSelect;
		FName       ToScrollTo = NAME_None;
		FName       ToRename   = NAME_None;
	};
	FPendingRowActions Pending;

	TWeakInterfacePtr<IMorphTargetManagerDataSource> WeakDataSource;
	FDelegateHandle InvalidationHandle;

	ESortMode SortMode = ESortMode::Alphabetical;
	bool bInvertFilter = false;
	bool bDisableFilter = false;

	// Commands
	TSharedPtr<FUICommandList> CommandList;
};

