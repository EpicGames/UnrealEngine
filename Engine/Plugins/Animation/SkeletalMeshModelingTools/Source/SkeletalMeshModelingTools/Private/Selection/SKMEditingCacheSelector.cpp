// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/SKMEditingCacheSelector.h"

#include "SkeletalMeshEditingCache.h"
#include "SkeletalMeshModelingToolsEditorMode.h"
#include "Components/SKMBackedDynaMeshComponent.h"
#include "DynamicSubmesh3.h"
#include "GroupTopology.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Selection/ToolSelectionUtil.h"

using namespace UE::Geometry;


// ============================================================================
// Selection remapping helper (editing-mesh IDs -> preview-mesh IDs)
// ============================================================================

namespace SkeletalMeshEditingCacheSelector::Private
{

enum class ERemapSelectionMode : uint8
{
	// Remap both the hard selection and the soft-selected vertices.
	IncludeSoftSelection,
	// Remap only the hard selection; leave SoftSelectedVertices empty.
	SelectionOnly
};

FGeometrySelection RemapSelectionToPreviewMesh(
	const FGeometrySelection& EditingSelection,
	const FDynamicSubmesh3* SubmeshMapping,
	const FGroupTopology* PreviewGroupTopology = nullptr,
	ERemapSelectionMode RemapMode = ERemapSelectionMode::IncludeSoftSelection)
{
	if (!SubmeshMapping)
	{
		return EditingSelection;
	}

	FGeometrySelection PreviewSelection;
	PreviewSelection.ElementType = EditingSelection.ElementType;
	PreviewSelection.TopologyType = EditingSelection.TopologyType;
	PreviewSelection.Selection.Reserve(EditingSelection.Selection.Num());

	for (uint64 EncodedID : EditingSelection.Selection)
	{
		FGeoSelectionID SelID(EncodedID);
		FGeoSelectionID PreviewSelID;
		bool bValid = false;

		if (EditingSelection.TopologyType == EGeometryTopologyType::Triangle)
		{
			switch (EditingSelection.ElementType)
			{
			case EGeometryElementType::Face:
			{
				int32 PreviewTID = SubmeshMapping->MapTriangleToSubmesh(static_cast<int32>(SelID.GeometryID));
				if (PreviewTID != FDynamicMesh3::InvalidID) { PreviewSelID = FGeoSelectionID::MeshTriangle(PreviewTID); bValid = true; }
				break;
			}
			case EGeometryElementType::Edge:
			{
				FMeshTriEdgeID TriEdge(SelID.GeometryID);
				int32 PreviewTID = SubmeshMapping->MapTriangleToSubmesh(static_cast<int32>(TriEdge.TriangleID));
				if (PreviewTID != FDynamicMesh3::InvalidID) { PreviewSelID = FGeoSelectionID::MeshEdge(FMeshTriEdgeID(PreviewTID, TriEdge.TriEdgeIndex)); bValid = true; }
				break;
			}
			case EGeometryElementType::Vertex:
			{
				int32 PreviewVID = SubmeshMapping->MapVertexToSubmesh(static_cast<int32>(SelID.GeometryID));
				if (PreviewVID != FDynamicMesh3::InvalidID) { PreviewSelID = FGeoSelectionID::MeshVertex(PreviewVID); bValid = true; }
				break;
			}
			}
		}
		else if (EditingSelection.TopologyType == EGeometryTopologyType::Polygroup)
		{
			switch (EditingSelection.ElementType)
			{
			case EGeometryElementType::Face:
			{
				int32 PreviewTID = SubmeshMapping->MapTriangleToSubmesh(static_cast<int32>(SelID.GeometryID));
				int32 PreviewGID = SubmeshMapping->MapGroupToSubmesh(static_cast<int32>(SelID.TopologyID));
				if (PreviewTID != FDynamicMesh3::InvalidID && PreviewGID != FDynamicMesh3::InvalidID)
				{
					PreviewSelID = FGeoSelectionID::GroupFace(PreviewTID, PreviewGID);
					bValid = true;
				}
				break;
			}
			case EGeometryElementType::Edge:
			{
				if (PreviewGroupTopology)
				{
					FMeshTriEdgeID EditingTriEdge(SelID.GeometryID);
					const FDynamicMesh3* BaseMesh = SubmeshMapping->GetBaseMesh();
					int32 EditingEdgeID = BaseMesh->GetTriEdge(EditingTriEdge.TriangleID, EditingTriEdge.TriEdgeIndex);
					int32 PreviewEdgeID = SubmeshMapping->MapEdgeToSubmesh(EditingEdgeID);
					if (PreviewEdgeID != FDynamicMesh3::InvalidID)
					{
						int32 PreviewGroupEdgeID = PreviewGroupTopology->FindGroupEdgeID(PreviewEdgeID);
						FMeshTriEdgeID PreviewTriEdge = SubmeshMapping->GetSubmesh().GetTriEdgeIDFromEdgeID(PreviewEdgeID);
						if (PreviewGroupEdgeID >= 0 && PreviewTriEdge.TriangleID != FDynamicMesh3::InvalidID)
						{
							PreviewSelID = FGeoSelectionID(PreviewTriEdge.Encoded(), PreviewGroupEdgeID);
							bValid = true;
						}
					}
				}
				break;
			}
			case EGeometryElementType::Vertex:
			{
				int32 PreviewVID = SubmeshMapping->MapVertexToSubmesh(static_cast<int32>(SelID.GeometryID));
				if (PreviewVID != FDynamicMesh3::InvalidID)
				{
					// CornerID differs between topologies — skip TopologyID, let consumer re-derive
					PreviewSelID = FGeoSelectionID(static_cast<uint32>(PreviewVID), 0);
					bValid = true;
				}
				break;
			}
			}
		}

		if (bValid)
		{
			PreviewSelection.Selection.Add(PreviewSelID.Encoded());
		}
	}

	if (RemapMode == ERemapSelectionMode::IncludeSoftSelection)
	{
		PreviewSelection.SoftSelectedVertices.Reserve(EditingSelection.SoftSelectedVertices.Num());
		for (const TPair<uint64, double>& Pair : EditingSelection.SoftSelectedVertices)
		{
			const int32 EditingVID = static_cast<int32>(FGeoSelectionID(Pair.Key).GeometryID);
			const int32 PreviewVID = SubmeshMapping->MapVertexToSubmesh(EditingVID);
			if (PreviewVID != FDynamicMesh3::InvalidID)
			{
				PreviewSelection.SoftSelectedVertices.Add(FGeoSelectionID::MeshVertex(PreviewVID).Encoded(), Pair.Value);
			}
		}
	}

	return PreviewSelection;
}


/**
 * Remap a single encoded selection element ID from preview-mesh space to editing-mesh space.
 * Returns the input unchanged when no conversion applies (e.g. polygroup-edge mapping fails).
 */
static FGeoSelectionID RemapPreviewIDToEditingMesh(
	FGeoSelectionID PreviewID,
	EGeometryTopologyType TopologyType,
	EGeometryElementType ElementType,
	const FDynamicSubmesh3* SubmeshMapping,
	UDynamicMesh* PreviewMesh,
	UDynamicMesh* InEditingMesh,
	const FGroupTopology* EditingTopology)
{
	if (TopologyType == EGeometryTopologyType::Triangle)
	{
		if (ElementType == EGeometryElementType::Face)
		{
			int32 EditingTID = SubmeshMapping->MapTriangleToBaseMesh(static_cast<int32>(PreviewID.GeometryID));
			return FGeoSelectionID::MeshTriangle(EditingTID);
		}
		else if (ElementType == EGeometryElementType::Edge)
		{
			FMeshTriEdgeID PreviewTriEdge(PreviewID.GeometryID);
			int32 EditingTID = SubmeshMapping->MapTriangleToBaseMesh(static_cast<int32>(PreviewTriEdge.TriangleID));
			return FGeoSelectionID::MeshEdge(FMeshTriEdgeID(EditingTID, PreviewTriEdge.TriEdgeIndex));
		}
		else if (ElementType == EGeometryElementType::Vertex)
		{
			int32 EditingVID = SubmeshMapping->MapVertexToBaseMesh(static_cast<int32>(PreviewID.GeometryID));
			return FGeoSelectionID::MeshVertex(EditingVID);
		}
	}
	else if (TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (ElementType == EGeometryElementType::Face)
		{
			int32 EditingTID = SubmeshMapping->MapTriangleToBaseMesh(static_cast<int32>(PreviewID.GeometryID));
			int32 EditingGroupID = SubmeshMapping->MapGroupToBaseMesh(static_cast<int32>(PreviewID.TopologyID));
			return FGeoSelectionID(static_cast<uint32>(EditingTID), static_cast<uint32>(EditingGroupID));
		}
		else if (ElementType == EGeometryElementType::Edge)
		{
			FMeshTriEdgeID PreviewTriEdge(PreviewID.GeometryID);
			int32 PreviewEdgeID = -1;
			PreviewMesh->ProcessMesh([&](const FDynamicMesh3& Mesh)
			{
				PreviewEdgeID = Mesh.IsTriangle(PreviewTriEdge.TriangleID) ?
					Mesh.GetTriEdge(PreviewTriEdge.TriangleID, PreviewTriEdge.TriEdgeIndex) : IndexConstants::InvalidID;
			});
			int32 EditingEID = SubmeshMapping->MapEdgeToBaseMesh(PreviewEdgeID);
			if (EditingEID != FDynamicMesh3::InvalidID && EditingTopology)
			{
				int32 EditingGroupEdgeID = EditingTopology->FindGroupEdgeID(EditingEID);
				FMeshTriEdgeID EditingTriEdgeID;
				InEditingMesh->ProcessMesh([&](const FDynamicMesh3& Mesh) { EditingTriEdgeID = Mesh.GetTriEdgeIDFromEdgeID(EditingEID); });
				return FGeoSelectionID(EditingTriEdgeID.Encoded(), static_cast<uint32>(EditingGroupEdgeID));
			}
		}
		else if (ElementType == EGeometryElementType::Vertex)
		{
			int32 EditingVID = SubmeshMapping->MapVertexToBaseMesh(static_cast<int32>(PreviewID.GeometryID));
			int32 EditingCornerID = EditingTopology ? EditingTopology->GetCornerIDFromVertexID(EditingVID) : 0;
			return FGeoSelectionID(static_cast<uint32>(EditingVID), static_cast<uint32>(EditingCornerID));
		}
	}
	return PreviewID;
}


/**
 * Remap encoded selection element IDs from preview-mesh space to editing-mesh space.
 * Handles all topology/element type combinations.
 */
static void RemapPreviewIDsToEditingMesh(
	TArray<uint64>& Elements,
	EGeometryTopologyType TopologyType,
	EGeometryElementType ElementType,
	const FDynamicSubmesh3* SubmeshMapping,
	UDynamicMesh* PreviewMesh,
	UDynamicMesh* InEditingMesh,
	const FGroupTopology* EditingTopology)
{
	for (uint64& Encoded : Elements)
	{
		Encoded = RemapPreviewIDToEditingMesh(FGeoSelectionID(Encoded), TopologyType, ElementType,
			SubmeshMapping, PreviewMesh, InEditingMesh, EditingTopology).Encoded();
	}
}

} // namespace SkeletalMeshEditingCacheSelector::Private

using namespace SkeletalMeshEditingCacheSelector::Private;


// ============================================================================
// Custom Transformers (remap selection in BeginTransform, then delegate to base)
// ============================================================================

void FSkeletalMeshEditingCacheSelectionTransformer::BeginTransform(const FGeometrySelection& Selection)
{
	FGeometrySelection PreviewSelection = RemapSelectionToPreviewMesh(Selection, IsolationSubmeshMapping, PreviewGroupTopology);
	FBasicDynamicMeshSelectionTransformer::BeginTransform(PreviewSelection);
}

void FSkeletalMeshEditingCachePolygroupTransformer::BeginTransform(const FGeometrySelection& Selection)
{
	FGeometrySelection PreviewSelection = RemapSelectionToPreviewMesh(Selection, IsolationSubmeshMapping, PreviewGroupTopology);
	FDynamicMeshPolygroupTransformer::BeginTransform(PreviewSelection);
}


// ============================================================================
// Lifecycle
// ============================================================================

FSkeletalMeshEditingCacheSelector::~FSkeletalMeshEditingCacheSelector() = default;

void FSkeletalMeshEditingCacheSelector::InitializeFromEditingCache(
	FGeometryIdentifier InSourceGeometryIdentifier,
	USkeletalMeshEditingCache* InEditingCache,
	TFunction<FName()> InGetEditingMorphTargetFunc)
{
	check(InEditingCache);

	WeakEditingCache = InEditingCache;
	GetEditingMorphTargetFunc = MoveTemp(InGetEditingMorphTargetFunc);

	// Store editing mesh reference (for topology/selection ID queries)
	USkeletalMeshBackedDynamicMeshComponent* EditingMeshComponent = InEditingCache->GetEditingMeshComponent();
	EditingMesh = EditingMeshComponent->GetDynamicMesh();
	EditingMeshIdentifier = FGeometryIdentifier::PrimitiveComponent(EditingMeshComponent, FGeometryIdentifier::EObjectType::DynamicMeshComponent);

	// Listen for editing mesh changes to invalidate editing topology cache
	EditingMesh_OnMeshChangedHandle = EditingMesh->OnMeshChanged().AddLambda(
		[this](UDynamicMesh*, FDynamicMeshChangeInfo)
		{
			InvalidateEditingMeshCaches();
		});

	// Base class: TargetMesh = PREVIEW mesh (for spatial queries, transformers, rendering)
	UDynamicMeshComponent* PreviewComponent = CastChecked<UDynamicMeshComponent>(InEditingCache->GetPreviewMeshComponent());
	TWeakObjectPtr<USkeletalMeshEditingCache> WeakCache = InEditingCache;
	Initialize(InSourceGeometryIdentifier, PreviewComponent->GetDynamicMesh(),
		[WeakCache]() -> FTransformSRT3d
		{
			return WeakCache.IsValid() ? FTransformSRT3d(WeakCache->GetTransform()) : FTransformSRT3d::Identity();
		});

	// Subscribe to isolation changes
	OnIsolationChangedHandle = InEditingCache->GetOnIsolationChanged().AddRaw(this, &FSkeletalMeshEditingCacheSelector::HandleIsolationChanged);

	// Set initial isolation state
	if (InEditingCache->HasIsolation())
	{
		SetActiveSubmeshMapping(InEditingCache->GetIsolationSubmesh());
	}
}

void FSkeletalMeshEditingCacheSelector::Shutdown()
{
	if (USkeletalMeshEditingCache* Cache = WeakEditingCache.Get())
	{
		Cache->GetOnIsolationChanged().Remove(OnIsolationChangedHandle);
		OnIsolationChangedHandle.Reset();
	}

	if (EditingMesh.IsValid() && EditingMesh_OnMeshChangedHandle.IsValid())
	{
		EditingMesh->OnMeshChanged().Remove(EditingMesh_OnMeshChangedHandle);
		EditingMesh_OnMeshChangedHandle.Reset();
	}

	EditingGroupTopology.Reset();
	EditingMesh = nullptr;
	ActiveSubmeshMapping = nullptr;

	FBaseDynamicMeshSelector::Shutdown();
}

void FSkeletalMeshEditingCacheSelector::SetActiveSubmeshMapping(const FDynamicSubmesh3* InSubmeshMapping)
{
	ActiveSubmeshMapping = InSubmeshMapping;
}



// ============================================================================
// Editing-mesh topology overrides
// ============================================================================

void FSkeletalMeshEditingCacheSelector::InvalidateEditingMeshCaches()
{
	EditingGroupTopology.Reset();
}

const FGroupTopology* FSkeletalMeshEditingCacheSelector::GetEditingMeshGroupTopology()
{
	if (!EditingGroupTopology.IsValid() && EditingMesh.IsValid())
	{
		EditingGroupTopology = MakePimpl<FGroupTopology>(EditingMesh->GetMeshPtr(), true);
	}
	return EditingGroupTopology.Get();
}

void FSkeletalMeshEditingCacheSelector::InitializeSelectionFromPredicate(
	FGeometrySelection& SelectionInOut,
	TFunctionRef<bool(FGeoSelectionID)> SelectionIDPredicate,
	EInitializeSelectionMode InitializeMode,
	const FGeometrySelection* ReferenceSelection)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::InitializeSelectionFromPredicate(SelectionInOut, SelectionIDPredicate, InitializeMode, ReferenceSelection);
		return;
	}

	if (InitializeMode != EInitializeSelectionMode::All && !ensure(ReferenceSelection != nullptr))
	{
		return;
	}

	const FGroupTopology* PreviewTopology = FBaseDynamicMeshSelector::GetGroupTopology();
	const FGroupTopology* EditingTopology = GetEditingMeshGroupTopology();

	FGeometrySelection PreviewReference;
	const FGeometrySelection* PreviewRefPtr = nullptr;
	if (ReferenceSelection)
	{
		PreviewReference = RemapSelectionToPreviewMesh(*ReferenceSelection, ActiveSubmeshMapping, PreviewTopology, ERemapSelectionMode::SelectionOnly);
		PreviewRefPtr = &PreviewReference;
	}

	// User predicate expects editing-mesh IDs; the base class iterates preview-mesh IDs.
	auto WrappedPredicate = [&](FGeoSelectionID PreviewID) -> bool
	{
		FGeoSelectionID EditingID = RemapPreviewIDToEditingMesh(PreviewID, SelectionInOut.TopologyType, SelectionInOut.ElementType,
			ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), EditingTopology);
		return SelectionIDPredicate(EditingID);
	};

	FGeometrySelection PreviewOut;
	PreviewOut.InitializeTypes(SelectionInOut.ElementType, SelectionInOut.TopologyType);
	FBaseDynamicMeshSelector::InitializeSelectionFromPredicate(PreviewOut, WrappedPredicate, InitializeMode, PreviewRefPtr);

	TArray<uint64> EditingIDs = PreviewOut.Selection.Array();
	RemapPreviewIDsToEditingMesh(EditingIDs, SelectionInOut.TopologyType, SelectionInOut.ElementType,
		ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), EditingTopology);
	SelectionInOut.Selection.Append(EditingIDs);
}

void FSkeletalMeshEditingCacheSelector::UpdateSelectionFromSelection(
	const FGeometrySelection& FromSelection,
	bool bAllowConversion,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionDelta* SelectionDelta)
{
	if (IsLockable() && IsLocked())
	{
		return;
	}

	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::UpdateSelectionFromSelection(FromSelection, bAllowConversion, SelectionEditor, UpdateConfig, SelectionDelta);
		return;
	}

	const FGroupTopology* PreviewTopology = FBaseDynamicMeshSelector::GetGroupTopology();
	const FGroupTopology* EditingTopology = GetEditingMeshGroupTopology();

	FGeometrySelection PreviewFromSelection = RemapSelectionToPreviewMesh(FromSelection, ActiveSubmeshMapping, PreviewTopology, ERemapSelectionMode::SelectionOnly);
	FGeometrySelection PreviewCurrentSelection = RemapSelectionToPreviewMesh(SelectionEditor.GetSelection(), ActiveSubmeshMapping, PreviewTopology, ERemapSelectionMode::SelectionOnly);
	FGeometrySelectionEditor TempEditor;
	TempEditor.Initialize(&PreviewCurrentSelection, SelectionEditor.GetTopologyType() == EGeometryTopologyType::Polygroup);

	FGeometrySelectionDelta TempDelta;
	FBaseDynamicMeshSelector::UpdateSelectionFromSelection(PreviewFromSelection, bAllowConversion, TempEditor, UpdateConfig, &TempDelta);

	if (TempDelta.IsEmpty())
	{
		return;
	}

	TArray<uint64> EditingSpaceIDs = PreviewCurrentSelection.Selection.Array();
	RemapPreviewIDsToEditingMesh(EditingSpaceIDs, SelectionEditor.GetTopologyType(), SelectionEditor.GetElementType(),
		ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), EditingTopology);
	UpdateSelectionWithNewElements(&SelectionEditor, EGeometrySelectionChangeType::Replace, EditingSpaceIDs, SelectionDelta);
}

bool FSkeletalMeshEditingCacheSelector::ConvertSelection(
	const FGeometrySelection& FromSelectionIn,
	FGeometrySelection& ToSelectionOut,
	const EEnumerateSelectionConversionParams ConversionParams)
{
	if (!ActiveSubmeshMapping)
	{
		return FBaseDynamicMeshSelector::ConvertSelection(FromSelectionIn, ToSelectionOut, ConversionParams);
	}

	const FGroupTopology* PreviewTopology = FBaseDynamicMeshSelector::GetGroupTopology();
	FGeometrySelection PreviewFrom = RemapSelectionToPreviewMesh(FromSelectionIn, ActiveSubmeshMapping, PreviewTopology, ERemapSelectionMode::SelectionOnly);
	FGeometrySelection PreviewTo;
	PreviewTo.InitializeTypes(ToSelectionOut.ElementType, ToSelectionOut.TopologyType);

	const bool bResult = FBaseDynamicMeshSelector::ConvertSelection(PreviewFrom, PreviewTo, ConversionParams);

	TArray<uint64> EditingIDs = PreviewTo.Selection.Array();
	RemapPreviewIDsToEditingMesh(EditingIDs, ToSelectionOut.TopologyType, ToSelectionOut.ElementType,
		ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), GetEditingMeshGroupTopology());
	ToSelectionOut.Selection.Append(EditingIDs);
	return bResult;
}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
void FSkeletalMeshEditingCacheSelector::UpdateSoftSelection(
	FGeometrySelection& Selection,
	const FUpdateSoftSelectionConfig& InSoftSelectionConfig)
{
	if (!ActiveSubmeshMapping)
	{
		// No isolation — IDs match, base class runs Dijkstra on preview mesh (= editing mesh topology)
		FBaseDynamicMeshSelector::UpdateSoftSelection(Selection, InSoftSelectionConfig);
		return;
	}

	// With isolation: remap selection to preview space, run Dijkstra on preview mesh, remap results back
	FGeometrySelection PreviewSelection = RemapSelectionToPreviewMesh(Selection, ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);
	FBaseDynamicMeshSelector::UpdateSoftSelection(PreviewSelection, InSoftSelectionConfig);

	// Remap soft-selected vertex IDs from preview space back to editing space
	Selection.SoftSelectedVertices.Reset();
	for (const TPair<uint64, double>& Pair : PreviewSelection.SoftSelectedVertices)
	{
		int32 PreviewVID = static_cast<int32>(FGeoSelectionID(Pair.Key).GeometryID);
		int32 EditingVID = RemapVertexToEditingMesh(PreviewVID);
		if (EditingVID != FDynamicMesh3::InvalidID)
		{
			Selection.SoftSelectedVertices.Add(FGeoSelectionID::MeshVertex(EditingVID).Encoded(), Pair.Value);
		}
	}
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS


// ============================================================================
// Rendering overrides (remap editing IDs to preview IDs, then use base/preview mesh)
// ============================================================================

void FSkeletalMeshEditingCacheSelector::AccumulateSelectionElements(
	const FGeometrySelection& Selection,
	FGeometrySelectionElements& Elements,
	bool bTransformToWorld,
	EEnumerateSelectionMapping Flags)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::AccumulateSelectionElements(Selection, Elements, bTransformToWorld, Flags);
		return;
	}

	FGeometrySelection PreviewSelection = RemapSelectionToPreviewMesh(Selection, ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);

	const FTransform UseWorldTransform = GetLocalToWorldTransform();
	const FTransform* ApplyTransform = bTransformToWorld ? &UseWorldTransform : nullptr;

	ToolSelectionUtil::AccumulateSelectionElements(
		Elements,
		PreviewSelection,
		TargetMesh->GetMeshRef(),
		PreviewSelection.TopologyType == EGeometryTopologyType::Polygroup ? FBaseDynamicMeshSelector::GetGroupTopology() : nullptr,
		ApplyTransform,
		Flags);
}

void FSkeletalMeshEditingCacheSelector::AccumulateSelectionBounds(
	const FGeometrySelection& Selection,
	FGeometrySelectionBounds& BoundsInOut,
	bool bTransformToWorld)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::AccumulateSelectionBounds(Selection, BoundsInOut, bTransformToWorld);
		return;
	}

	FGeometrySelection PreviewSelection = RemapSelectionToPreviewMesh(Selection, ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);

	FTransform UseTransform = bTransformToWorld ? GetLocalToWorldTransform() : FTransform::Identity;
	FAxisAlignedBox3d TargetWorldBounds = FAxisAlignedBox3d::Empty();
	int32 ElementCount = 0;

	if (PreviewSelection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
		{
			EnumeratePolygroupSelectionVertices(PreviewSelection, SourceMesh, FBaseDynamicMeshSelector::GetGroupTopology(), UseTransform,
				[&](uint32 VertexID, const FVector3d& Position) { TargetWorldBounds.Contain(Position); ElementCount++; });
		});
	}
	else
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
		{
			EnumerateTriangleSelectionVertices(PreviewSelection, SourceMesh, &UseTransform,
				[&](uint32 VertexID, const FVector3d& Position) { TargetWorldBounds.Contain(Position); ElementCount++; });
		});
	}

	if (ElementCount > 0)
	{
		BoundsInOut.WorldBounds.Contain(TargetWorldBounds);
	}
}

void FSkeletalMeshEditingCacheSelector::GetSelectionFrame(
	const FGeometrySelection& Selection,
	FFrame3d& SelectionFrame,
	bool bTransformToWorld)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::GetSelectionFrame(Selection, SelectionFrame, bTransformToWorld);
		return;
	}

	FGeometrySelection PreviewSelection = RemapSelectionToPreviewMesh(Selection, ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);

	if (PreviewSelection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		const FGroupTopology* Topology = FBaseDynamicMeshSelector::GetGroupTopology();
		FGroupTopologySelection TopoSelection;
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
		{
			ConvertPolygroupSelectionToTopologySelection(PreviewSelection, SourceMesh, Topology, TopoSelection);
		});
		SelectionFrame = Topology->GetSelectionFrame(TopoSelection);
	}
	else
	{
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
		{
			GetTriangleSelectionFrame(PreviewSelection, SourceMesh, SelectionFrame);
		});
	}

	if (bTransformToWorld)
	{
		SelectionFrame.Transform(GetLocalToWorldTransform());
	}
}


// ============================================================================
// Raycast overrides (base hits preview mesh, remap output IDs to editing space)
// ============================================================================

void FSkeletalMeshEditingCacheSelector::UpdateSelectionViaRaycast(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::UpdateSelectionViaRaycast(RayInfo, SelectionEditor, UpdateConfig, ResultOut);
		return;
	}

	if (IsLockable() && IsLocked())
	{
		return;
	}

	if (SelectionEditor.GetTopologyType() == EGeometryTopologyType::Triangle)
	{
		UpdateSelectionViaRaycast_MeshTopology(RayInfo, SelectionEditor, UpdateConfig, ResultOut);
		return;
	}

	check(SelectionEditor.GetTopologyType() == EGeometryTopologyType::Polygroup);

	if (SelectionEditor.GetElementType() == EGeometryElementType::Edge)
	{
		UpdateSelectionViaRaycast_GroupEdges(RayInfo, SelectionEditor, UpdateConfig, ResultOut);
	}
	else
	{
		// Raycast into a temporary selection in preview-space, then remap final state and replace real selection.
		FGeometrySelection TempSelection = RemapSelectionToPreviewMesh(
			SelectionEditor.GetSelection(), ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);
		FGeometrySelectionEditor TempEditor;
		TempEditor.Initialize(&TempSelection, true);

		FRay3d LocalRay = GetWorldTransformFunc().InverseTransformRay(RayInfo.WorldRay);
		FGeometrySelectionUpdateResult TempResult;
		UpdateGroupSelectionViaRaycast(
			GetColliderMesh(), FBaseDynamicMeshSelector::GetGroupTopology(), &TempEditor,
			LocalRay, UpdateConfig, TempResult);

		ResultOut.bSelectionModified = TempResult.bSelectionModified;
		ResultOut.bSelectionMissed = TempResult.bSelectionMissed;
		if (!TempResult.bSelectionMissed)
		{
			TArray<uint64> EditingSpaceIDs = TempSelection.Selection.Array();
			RemapPreviewIDsToEditingMesh(EditingSpaceIDs,
				SelectionEditor.GetTopologyType(), SelectionEditor.GetElementType(),
				ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), GetEditingMeshGroupTopology());

			UpdateSelectionWithNewElements(
				&SelectionEditor, EGeometrySelectionChangeType::Replace,
				EditingSpaceIDs, &ResultOut.SelectionDelta);
		}
	}
}

void FSkeletalMeshEditingCacheSelector::UpdateSelectionViaRaycast_MeshTopology(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::UpdateSelectionViaRaycast_MeshTopology(RayInfo, SelectionEditor, UpdateConfig, ResultOut);
		return;
	}

	// Raycast into a temp selection in preview-space, then remap and apply to real selection.
	// Initialize temp selection with current selection remapped to preview space so removal deltas work.
	FGeometrySelection TempSelection = RemapSelectionToPreviewMesh(
		SelectionEditor.GetSelection(), ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);
	FGeometrySelectionEditor TempEditor;
	TempEditor.Initialize(&TempSelection, false);

	FRay3d LocalRay = GetWorldTransformFunc().InverseTransformRay(RayInfo.WorldRay);
	FGeometrySelectionUpdateResult TempResult;
	UpdateTriangleSelectionViaRaycast(GetColliderMesh(), &TempEditor, LocalRay, UpdateConfig, TempResult);

	ResultOut.bSelectionModified = TempResult.bSelectionModified;
	ResultOut.bSelectionMissed = TempResult.bSelectionMissed;
	if (!TempResult.bSelectionMissed)
	{
		TArray<uint64> EditingSpaceIDs = TempSelection.Selection.Array();
		RemapPreviewIDsToEditingMesh(EditingSpaceIDs,
			EGeometryTopologyType::Triangle, SelectionEditor.GetElementType(),
			ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), GetEditingMeshGroupTopology());

		UpdateSelectionWithNewElements(
			&SelectionEditor, EGeometrySelectionChangeType::Replace,
			EditingSpaceIDs, &ResultOut.SelectionDelta);
	}

	// Handle dual TriEdgeIDs for edge selection — use editing mesh for topology lookup
	if (ResultOut.bSelectionModified && SelectionEditor.GetElementType() == EGeometryElementType::Edge)
	{
		auto AffectBothTriEdgeIDs = [this, &SelectionEditor, &ResultOut](const EGeometrySelectionChangeType ChangeType)
		{
			const TArray<uint64>& DeltaElements = (ChangeType == EGeometrySelectionChangeType::Add ? ResultOut.SelectionDelta.Added : ResultOut.SelectionDelta.Removed);
			TArray<uint64> ElementsToUpdate;
			for (const uint64 Element : DeltaElements)
			{
				FMeshTriEdgeID TriEdgeID(FGeoSelectionID(Element).GeometryID);
				EditingMesh->ProcessMesh([TriEdgeID, &ElementsToUpdate](const FDynamicMesh3& SourceMesh)
				{
					const int32 EdgeID = SourceMesh.IsTriangle(TriEdgeID.TriangleID) ? SourceMesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
					if (SourceMesh.IsEdge(EdgeID))
					{
						SourceMesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID,
							[TriEdgeID, &ElementsToUpdate](const FMeshTriEdgeID OtherTriEdgeID)
							{
								if (OtherTriEdgeID.TriangleID != TriEdgeID.TriangleID)
								{
									ElementsToUpdate.Add(OtherTriEdgeID.Encoded());
								}
							});
					}
				});
			}
			UpdateSelectionWithNewElements(&SelectionEditor, ChangeType, ElementsToUpdate, &ResultOut.SelectionDelta);
		};
		AffectBothTriEdgeIDs(EGeometrySelectionChangeType::Remove);
		AffectBothTriEdgeIDs(EGeometrySelectionChangeType::Add);
	}
}

void FSkeletalMeshEditingCacheSelector::UpdateSelectionViaRaycast_GroupEdges(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::UpdateSelectionViaRaycast_GroupEdges(RayInfo, SelectionEditor, UpdateConfig, ResultOut);
		return;
	}

	// Raycast into temp selection in preview-space, then remap and apply.
	// Seed with current selection remapped to preview space so removal deltas work.
	FGeometrySelection TempSelection = RemapSelectionToPreviewMesh(
		SelectionEditor.GetSelection(), ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);
	FGeometrySelectionEditor TempEditor;
	TempEditor.Initialize(&TempSelection, true);

	FGeometrySelectionUpdateResult TempResult;
	FBaseDynamicMeshSelector::UpdateSelectionViaRaycast_GroupEdges(RayInfo, TempEditor, UpdateConfig, TempResult);

	ResultOut.bSelectionModified = TempResult.bSelectionModified;
	ResultOut.bSelectionMissed = TempResult.bSelectionMissed;
	if (!TempResult.bSelectionMissed)
	{
		TArray<uint64> EditingSpaceIDs = TempSelection.Selection.Array();
		RemapPreviewIDsToEditingMesh(EditingSpaceIDs,
			EGeometryTopologyType::Polygroup, EGeometryElementType::Edge,
			ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), GetEditingMeshGroupTopology());

		UpdateSelectionWithNewElements(
			&SelectionEditor, EGeometrySelectionChangeType::Replace,
			EditingSpaceIDs, &ResultOut.SelectionDelta);
	}
}

void FSkeletalMeshEditingCacheSelector::GetSelectionPreviewForRaycast(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& PreviewEditor)
{
	if (IsLockable() && IsLocked())
	{
		return;
	}

	FGeometrySelectionUpdateResult UpdateResult;
	UpdateSelectionViaRaycast(RayInfo, PreviewEditor, FGeometrySelectionUpdateConfig(), UpdateResult);
}

void FSkeletalMeshEditingCacheSelector::UpdateSelectionViaShape(
	const FWorldShapeQueryInfo& ShapeInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	if (!ActiveSubmeshMapping)
	{
		FBaseDynamicMeshSelector::UpdateSelectionViaShape(ShapeInfo, SelectionEditor, UpdateConfig, ResultOut);
		return;
	}

	// Shape-select into temp selection in preview-space, then remap and apply.
	// Seed with current selection remapped to preview space so removal deltas work.
	FGeometrySelection TempSelection = RemapSelectionToPreviewMesh(
		SelectionEditor.GetSelection(), ActiveSubmeshMapping, FBaseDynamicMeshSelector::GetGroupTopology(), ERemapSelectionMode::SelectionOnly);
	FGeometrySelectionEditor TempEditor;
	TempEditor.Initialize(&TempSelection, SelectionEditor.GetTopologyType() == EGeometryTopologyType::Polygroup);

	FGeometrySelectionUpdateResult TempResult;
	FBaseDynamicMeshSelector::UpdateSelectionViaShape(ShapeInfo, TempEditor, UpdateConfig, TempResult);

	ResultOut.bSelectionModified = TempResult.bSelectionModified;
	ResultOut.bSelectionMissed = TempResult.bSelectionMissed;
	if (!TempResult.bSelectionMissed)
	{
		TArray<uint64> EditingSpaceIDs = TempSelection.Selection.Array();
		RemapPreviewIDsToEditingMesh(EditingSpaceIDs,
			SelectionEditor.GetTopologyType(), SelectionEditor.GetElementType(),
			ActiveSubmeshMapping, TargetMesh.Get(), EditingMesh.Get(), GetEditingMeshGroupTopology());

		UpdateSelectionWithNewElements(
			&SelectionEditor, EGeometrySelectionChangeType::Replace,
			EditingSpaceIDs, &ResultOut.SelectionDelta);
	}
}


// ============================================================================
// ID Remapping Helpers
// ============================================================================

int32 FSkeletalMeshEditingCacheSelector::RemapVertexToEditingMesh(int32 PreviewVertexID) const
{
	return ActiveSubmeshMapping ? ActiveSubmeshMapping->MapVertexToBaseMesh(PreviewVertexID) : PreviewVertexID;
}

int32 FSkeletalMeshEditingCacheSelector::RemapTriangleToEditingMesh(int32 PreviewTriangleID) const
{
	return ActiveSubmeshMapping ? ActiveSubmeshMapping->MapTriangleToBaseMesh(PreviewTriangleID) : PreviewTriangleID;
}

int32 FSkeletalMeshEditingCacheSelector::RemapEdgeToEditingMesh(int32 PreviewEdgeID) const
{
	return ActiveSubmeshMapping ? ActiveSubmeshMapping->MapEdgeToBaseMesh(PreviewEdgeID) : PreviewEdgeID;
}

int32 FSkeletalMeshEditingCacheSelector::RemapVertexToPreviewMesh(int32 EditingVertexID) const
{
	return ActiveSubmeshMapping ? ActiveSubmeshMapping->MapVertexToSubmesh(EditingVertexID) : EditingVertexID;
}

int32 FSkeletalMeshEditingCacheSelector::RemapTriangleToPreviewMesh(int32 EditingTriangleID) const
{
	return ActiveSubmeshMapping ? ActiveSubmeshMapping->MapTriangleToSubmesh(EditingTriangleID) : EditingTriangleID;
}


// ============================================================================
// Isolation Change Handling
// ============================================================================

void FSkeletalMeshEditingCacheSelector::HandleIsolationChanged()
{
	USkeletalMeshEditingCache* Cache = WeakEditingCache.Get();
	ActiveSubmeshMapping = Cache ? Cache->GetIsolationSubmesh() : nullptr;
	// Base class spatial caches and NotifyGeometryModified() are already handled by
	// TargetMesh->OnMeshChanged, which fires when RebuildPreviewMesh replaces the preview mesh content.
}


// ============================================================================
// Transformation
// ============================================================================

IGeometrySelectionTransformer* FSkeletalMeshEditingCacheSelector::InitializeTransformation(const FGeometrySelection& Selection)
{
	check(!ActiveTransformer);

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		auto Transformer = MakeShared<FSkeletalMeshEditingCachePolygroupTransformer>();
		Transformer->IsolationSubmeshMapping = ActiveSubmeshMapping;
		Transformer->PreviewGroupTopology = FBaseDynamicMeshSelector::GetGroupTopology();
		ActiveTransformer = Transformer;
	}
	else
	{
		auto Transformer = MakeShared<FSkeletalMeshEditingCacheSelectionTransformer>();
		Transformer->IsolationSubmeshMapping = ActiveSubmeshMapping;
		Transformer->PreviewGroupTopology = FBaseDynamicMeshSelector::GetGroupTopology();
		ActiveTransformer = Transformer;
	}

	ActiveTransformer->bEnableSelectionTransformDrawing = false;
	ActiveTransformer->Initialize(this);
	ActiveTransformer->OnEndTransformFunc = [this](IToolsContextTransactionsAPI*)
	{
		CommitMeshTransform();
	};
	return ActiveTransformer.Get();
}

void FSkeletalMeshEditingCacheSelector::ShutdownTransformation(IGeometrySelectionTransformer* Transformer)
{
	ActiveTransformer.Reset();
}

void FSkeletalMeshEditingCacheSelector::CommitMeshTransform()
{
	USkeletalMeshEditingCache* Cache = WeakEditingCache.Get();
	if (!Cache)
	{
		return;
	}

	FName MorphTargetName = GetEditingMorphTargetFunc ? GetEditingMorphTargetFunc() : NAME_None;
	// TargetMesh is the preview mesh — pass it to HandleGeometryUpdate which unposes changes back to editing mesh
	TargetMesh->ProcessMesh([&](const FDynamicMesh3& PreviewMesh)
	{
		Cache->HandleGeometryUpdate(PreviewMesh, MorphTargetName);
		Cache->RequestDeformPreviewMesh();
	});
}


// ============================================================================
// Factory
// ============================================================================

void FSkeletalMeshEditingCacheSelectorFactory::Init(USkeletalMeshModelingToolsEditorMode* InEditorMode)
{
	WeakEditorMode = InEditorMode;
}

bool FSkeletalMeshEditingCacheSelectorFactory::CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	if (USkeletalMeshModelingToolsEditorMode* EditorMode = WeakEditorMode.Get())
	{
		if (EditorMode->GetCurrentEditingCache())
		{
			return true;
		}
	}
	return false;
}

TUniquePtr<IGeometrySelector> FSkeletalMeshEditingCacheSelectorFactory::BuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	USkeletalMeshModelingToolsEditorMode* EditorMode = WeakEditorMode.Get();
	check(EditorMode);
	USkeletalMeshEditingCache* Cache = EditorMode->GetCurrentEditingCache();
	check(Cache);

	TUniquePtr<FSkeletalMeshEditingCacheSelector> Selector = MakeUnique<FSkeletalMeshEditingCacheSelector>();

	TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> WeakMode = EditorMode;
	Selector->InitializeFromEditingCache(
		TargetIdentifier,
		Cache,
		[WeakMode]() { return WeakMode.IsValid() ? WeakMode->GetEditingMorphTarget() : NAME_None; }
	);

	return Selector;
}
