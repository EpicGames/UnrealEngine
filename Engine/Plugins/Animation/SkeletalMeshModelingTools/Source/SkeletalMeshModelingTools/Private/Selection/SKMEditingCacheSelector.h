// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Selection/DynamicMeshSelector.h"
#include "Selection/DynamicMeshPolygroupTransformer.h"
#include "Templates/PimplPtr.h"

#define UE_API SKELETALMESHMODELINGTOOLS_API

namespace UE::Geometry
{
	struct FDynamicSubmesh3;
	class FGroupTopology;
}

class USkeletalMeshEditingCache;
class USkeletalMeshModelingToolsEditorMode;

/**
 * Custom transformers that remap selection from editing-mesh space to preview-mesh space
 * before delegating to the base class. Only BeginTransform is overridden.
 */
class FSkeletalMeshEditingCacheSelectionTransformer : public FBasicDynamicMeshSelectionTransformer
{
public:
	const UE::Geometry::FDynamicSubmesh3* IsolationSubmeshMapping = nullptr;
	const UE::Geometry::FGroupTopology* PreviewGroupTopology = nullptr;
	UE_API virtual void BeginTransform(const UE::Geometry::FGeometrySelection& Selection) override;
};

class FSkeletalMeshEditingCachePolygroupTransformer : public FDynamicMeshPolygroupTransformer
{
public:
	const UE::Geometry::FDynamicSubmesh3* IsolationSubmeshMapping = nullptr;
	const UE::Geometry::FGroupTopology* PreviewGroupTopology = nullptr;
	UE_API virtual void BeginTransform(const UE::Geometry::FGeometrySelection& Selection) override;
};


/**
 * Dual-mesh selector for skeletal mesh editing.
 *
 * Base-class TargetMesh is the posed preview mesh (from UPreviewMesh's DynamicMeshComponent).
 * Spatial queries, hit testing, transformers, and rendering all use the preview mesh naturally.
 *
 * Selection IDs are always stored in editing-mesh (full mesh) space. When geometry isolation is
 * active, the preview mesh is a submesh with different IDs, so the selector remaps between the
 * two spaces via FDynamicSubmesh3 mappings. When no isolation is active, IDs are identical and
 * no remapping is needed.
 *
 * A separate EditingMesh reference provides access to the full unposed mesh for topology queries
 * and selection ID construction.
 */
class FSkeletalMeshEditingCacheSelector : public FBaseDynamicMeshSelector
{
public:
	UE_API virtual ~FSkeletalMeshEditingCacheSelector();

	/** Initialize from an editing cache. TargetMesh = preview mesh, stores editing mesh separately. */
	UE_API void InitializeFromEditingCache(
		FGeometryIdentifier InSourceGeometryIdentifier,
		USkeletalMeshEditingCache* InEditingCache,
		TFunction<FName()> InGetEditingMorphTargetFunc);

	/** Set the active submesh mapping for ID remapping. nullptr when isolation is not active. */
	UE_API void SetActiveSubmeshMapping(const UE::Geometry::FDynamicSubmesh3* InSubmeshMapping);

	// --- Lifecycle ---

	UE_API virtual void Shutdown() override;

	// --- Selection topology overrides (use EditingMesh) ---

	UE_API virtual void InitializeSelectionFromPredicate(
		FGeometrySelection& SelectionInOut,
		TFunctionRef<bool(UE::Geometry::FGeoSelectionID)> SelectionIDPredicate,
		EInitializeSelectionMode InitializeMode = EInitializeSelectionMode::All,
		const FGeometrySelection* ReferenceSelection = nullptr) override;

	UE_API virtual void UpdateSelectionFromSelection(
		const FGeometrySelection& FromSelection,
		bool bAllowConversion,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		UE::Geometry::FGeometrySelectionDelta* SelectionDelta = nullptr) override;

	UE_API virtual bool ConvertSelection(
		const FGeometrySelection& FromSelectionIn,
		FGeometrySelection& ToSelectionOut,
		const UE::Geometry::EEnumerateSelectionConversionParams ConversionParams) override;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	UE_API virtual void UpdateSoftSelection(
		FGeometrySelection& Selection,
		const FUpdateSoftSelectionConfig& InSoftSelectionConfig) override;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	// --- Rendering overrides (remap editing IDs to preview IDs for visualization) ---

	UE_API virtual void AccumulateSelectionElements(
		const FGeometrySelection& Selection,
		FGeometrySelectionElements& Elements,
		bool bTransformToWorld,
		UE::Geometry::EEnumerateSelectionMapping Flags = UE::Geometry::EEnumerateSelectionMapping::Default) override;

	UE_API virtual void AccumulateSelectionBounds(
		const FGeometrySelection& Selection,
		FGeometrySelectionBounds& BoundsInOut,
		bool bTransformToWorld) override;

	UE_API virtual void GetSelectionFrame(
		const FGeometrySelection& Selection,
		UE::Geometry::FFrame3d& SelectionFrame,
		bool bTransformToWorld) override;

	// --- Raycast overrides (hit preview mesh, emit editing-mesh IDs) ---

	UE_API virtual void UpdateSelectionViaRaycast(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut) override;

	UE_API virtual void GetSelectionPreviewForRaycast(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& PreviewEditor) override;

	UE_API virtual void UpdateSelectionViaShape(
		const FWorldShapeQueryInfo& ShapeInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut) override;

	// --- Transformation ---

	UE_API virtual IGeometrySelectionTransformer* InitializeTransformation(const FGeometrySelection& Selection) override;
	UE_API virtual void ShutdownTransformation(IGeometrySelectionTransformer* Transformer) override;

protected:
	// --- Raycast sub-method overrides ---

	UE_API virtual void UpdateSelectionViaRaycast_MeshTopology(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut) override;

	UE_API virtual void UpdateSelectionViaRaycast_GroupEdges(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut) override;

	// --- Editing Mesh State ---

	TWeakObjectPtr<UDynamicMesh> EditingMesh;
	FDelegateHandle EditingMesh_OnMeshChangedHandle;
	FGeometryIdentifier EditingMeshIdentifier;
	TPimplPtr<UE::Geometry::FGroupTopology> EditingGroupTopology;

	UE_API const UE::Geometry::FGroupTopology* GetEditingMeshGroupTopology();
	UE_API void InvalidateEditingMeshCaches();

	// --- Submesh Mapping ---

	/** Non-owning pointer to the submesh mapping. nullptr when isolation is not active. */
	const UE::Geometry::FDynamicSubmesh3* ActiveSubmeshMapping = nullptr;

	// --- ID Remapping Helpers ---

	/** Remap from preview-mesh space to editing-mesh space. Identity when no submesh mapping active. */
	int32 RemapVertexToEditingMesh(int32 PreviewVertexID) const;
	int32 RemapTriangleToEditingMesh(int32 PreviewTriangleID) const;
	int32 RemapEdgeToEditingMesh(int32 PreviewEdgeID) const;

	/** Remap from editing-mesh space to preview-mesh space. Returns InvalidID if not in submesh. */
	int32 RemapVertexToPreviewMesh(int32 EditingVertexID) const;
	int32 RemapTriangleToPreviewMesh(int32 EditingTriangleID) const;

	// --- Isolation Change Handling ---

	FDelegateHandle OnIsolationChangedHandle;
	UE_API void HandleIsolationChanged();

	// --- Transformation State ---

	TWeakObjectPtr<USkeletalMeshEditingCache> WeakEditingCache;
	TFunction<FName()> GetEditingMorphTargetFunc;
	TSharedPtr<FBasicDynamicMeshSelectionTransformer> ActiveTransformer;
	UE_API void CommitMeshTransform();

	friend class FSkeletalMeshEditingCacheSelectorFactory;
};


/**
 * Factory that constructs FSkeletalMeshEditingCacheSelector instances.
 */
class FSkeletalMeshEditingCacheSelectorFactory : public IGeometrySelectorFactory
{
public:
	UE_API void Init(USkeletalMeshModelingToolsEditorMode* InEditorMode);

	UE_API virtual bool CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const override;
	UE_API virtual TUniquePtr<IGeometrySelector> BuildForTarget(FGeometryIdentifier TargetIdentifier) const override;

protected:
	TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> WeakEditorMode;
};

#undef UE_API
