// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/SKMGeometrySelectionTransformTweaker.h"

#include "InteractiveToolsContext.h"
#include "ModelingSelectionInteraction.h"
#include "SkeletalMeshGizmoUtils.h"
#include "BaseGizmos/TransformProxy.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/TransformGizmo.h"
#include "Selection/GeometrySelectionManager.h"

#define LOCTEXT_NAMESPACE "USkeletalMeshGeometrySelectionTransformManipulator"

using namespace UE::Geometry;

void USkeletalMeshGeometrySelectionTransformTweaker::Setup(UGeometrySelectionManager* InSelectionManager)
{
	SelectionManager = InSelectionManager;

	SelectionManager->OnSelectionModified.AddUObject(this, &USkeletalMeshGeometrySelectionTransformTweaker::OnSelectionManager_SelectionModified);
}

void USkeletalMeshGeometrySelectionTransformTweaker::Shutdown()
{
	if (SelectionManager.IsValid())
	{
		SelectionManager->OnSelectionModified.RemoveAll(this);
	}
}

const FTransform& USkeletalMeshGeometrySelectionTransformTweaker::GetSelectionFrameTransform()
{
	return CachedSelectionFrameTransform;
}

void USkeletalMeshGeometrySelectionTransformTweaker::BeginTransformEdit()
{
	StartSelectionFrameTransform = CachedSelectionFrameTransform;
	Scale = FVector::One();
	// Sometimes when moving the 5.8 gizmo using indirect manipulation (ctrl left click)
	// BeginTransformEdit can be called twice in the same frame, coming from
	// UViewportClickDragBehavior::BeginCapture & UpdateCapture. To be investigated
	if (!SelectionManager->IsInActiveTransformation())
	{
		SelectionManager->BeginTransformation();
	}
}

void USkeletalMeshGeometrySelectionTransformTweaker::EndTransformEdit()
{
	SelectionManager->EndTransformation();
	
	UpdateSelectionFrameTransform();
}

bool USkeletalMeshGeometrySelectionTransformTweaker::IsEditingTransform() const
{
	return SelectionManager->IsInActiveTransformation();
}

void USkeletalMeshGeometrySelectionTransformTweaker::TweakTransform(FVector& InDrag, FRotator& InRot, FVector& InScale, bool bInIsWorldSpace)
{
	CachedSelectionFrameTransform.AddToTranslation(InDrag);
	CachedSelectionFrameTransform.SetRotation(InRot.Quaternion() * CachedSelectionFrameTransform.GetRotation());
	Scale += InScale;

	FQuat RotationToApply = CachedSelectionFrameTransform.GetRotation() * StartSelectionFrameTransform.GetRotation().Inverse();
	FVector TranslationToApply = CachedSelectionFrameTransform.GetLocation() - StartSelectionFrameTransform.GetLocation();
	FVector ScaleToApply = Scale - 1;

	SelectionManager->UpdateTransformation(StartSelectionFrameTransform, TranslationToApply, RotationToApply, ScaleToApply, bInIsWorldSpace);
	
}

void USkeletalMeshGeometrySelectionTransformTweaker::SetLocalFrameMode(EModelingSelectionInteraction_LocalFrameMode InLocalFrameMode)
{
	if (LocalFrameMode != InLocalFrameMode)
	{
		LocalFrameMode = InLocalFrameMode;
		UpdateSelectionFrameTransform();
	}
}

EModelingSelectionInteraction_LocalFrameMode USkeletalMeshGeometrySelectionTransformTweaker::GetLocalFrameMode() const
{
	return LocalFrameMode;
}

void USkeletalMeshGeometrySelectionTransformTweaker::OnSelectionManager_SelectionModified()
{
	UpdateSelectionFrameTransform();	
}

void USkeletalMeshGeometrySelectionTransformTweaker::UpdateSelectionFrameTransform()
{
	if (!SelectionManager->HasSelection())
	{
		return;
	}
	
	FFrame3d SelectionFrame;

	if (LocalFrameMode == EModelingSelectionInteraction_LocalFrameMode::FromGeometry)
	{
		SelectionManager->GetSelectionWorldFrame(SelectionFrame);
	}
	else
	{
		SelectionManager->GetTargetWorldFrame(SelectionFrame);
	}
	
	CachedSelectionFrameTransform = SelectionFrame.ToFTransform();
}

#undef LOCTEXT_NAMESPACE 

