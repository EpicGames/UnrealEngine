// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingSelectionInteraction.h"

#include "SKMGeometrySelectionTransformTweaker.generated.h"

class UGeometrySelectionManager;
class UTransformProxy;
class UTransformGizmo;


UCLASS(MinimalAPI)
class USkeletalMeshGeometrySelectionTransformTweaker : public UObject
{
	GENERATED_BODY()
	
public:
	void Setup(UGeometrySelectionManager* InSelectionManager);
	void Shutdown();

	const FTransform& GetSelectionFrameTransform();

	void BeginTransformEdit();
	void EndTransformEdit();
	bool IsEditingTransform() const;
	void TweakTransform(FVector& InDrag, FRotator& InRot, FVector& InScale, bool bInIsWorldSpace);

	void SetLocalFrameMode(EModelingSelectionInteraction_LocalFrameMode InLocalFrameMode);
	EModelingSelectionInteraction_LocalFrameMode GetLocalFrameMode() const;
	
	void UpdateSelectionFrameTransform();
protected:

	void OnSelectionManager_SelectionModified();
	
	TWeakObjectPtr<UGeometrySelectionManager> SelectionManager;

	FTransform CachedSelectionFrameTransform;
	
	FTransform StartSelectionFrameTransform;

	FVector Scale;

	EModelingSelectionInteraction_LocalFrameMode LocalFrameMode = EModelingSelectionInteraction_LocalFrameMode::FromGeometry;
};