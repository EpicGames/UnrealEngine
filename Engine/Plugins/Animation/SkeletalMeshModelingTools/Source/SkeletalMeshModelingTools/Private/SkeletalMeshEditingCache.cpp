// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshEditingCache.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "PreviewMesh.h"
#include "SkeletalDebugRendering.h"
#include "SkeletalMeshEditorUtils.h"
#include "Components/SKMBackedDynaMeshComponent.h"
#include "Preferences/PersonaOptions.h"
#include "SkeletalMesh/RefSkeletonPoser.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "Components/DynamicMeshComponent.h"
#include "Animation/AnimInstance.h"
#include "DynamicMesh/MeshNormals.h"
#include "UnrealClient.h"
#include "EditorViewportClient.h"
#include "SceneView.h"
#include "Engine/Engine.h"
#include "Selections/GeometrySelectionUtil.h"
#include "InteractiveToolChange.h"
#include "ToolContextInterfaces.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshEditingCache)

#define LOCTEXT_NAMESPACE "SkeletalMeshEditingCache"

namespace SkeletalMeshEditingCacheLocal
{
	static FIntPoint GetDPIUnscaledSize(FViewport* Viewport, FViewportClient* Client)
	{
		const FIntPoint Size = Viewport->GetSizeXY();
		const float DPIScale = Client->GetDPIScale();
		// (FIntPoint / float) implicitly casts the float to an int if you try to divide it directly
		return FIntPoint(static_cast<int32>(Size.X / DPIScale), static_cast<int32>(Size.Y / DPIScale));
	}
}

FSkeletalMeshEditingCacheNotifier::FSkeletalMeshEditingCacheNotifier(USkeletalMeshEditingCache* InEditingCahe)
	:EditingCache(InEditingCahe)
{
}

void FSkeletalMeshEditingCacheNotifier::HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	if (Notifying())
	{
		return;
	}

	EditingCache->HandleNotification(BoneNames, InNotifyType);

	Notify(BoneNames, InNotifyType);
}

void USkeletalMeshEditingCache::Spawn(UWorld* World, USkeletalMeshComponent* InSkeletalMeshComponent, EMeshLODIdentifier InLOD, const FDelegates& InDelegates, IToolsContextTransactionsAPI* InTransactionsAPI)
{
	SetFlags(RF_Transactional);
	
	Delegates = InDelegates;
	TransactionsAPI = InTransactionsAPI;
	SkeletalMeshComponent = InSkeletalMeshComponent;
	
	HostActor = World->SpawnActor(AActor::StaticClass());
	EditingMeshComponent = NewObject<USkeletalMeshBackedDynamicMeshComponent>(HostActor);

	USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	EditingMeshComponent->Init(SkeletalMesh, InLOD);

	TArray<UMaterialInterface*> Materials = SkeletalMeshComponent->GetMaterials();
	for (int k = 0; k < Materials.Num(); ++k)
	{
		EditingMeshComponent->SetMaterial(k, Materials[k]);
	}
	
	EditingMeshComponent->GetOnRequestingVisibilityChange().AddUObject(this, &USkeletalMeshEditingCache::HandleVisibilityChangeRequest);
	EditingMeshComponent->OnChanged().AddUObject(this, &USkeletalMeshEditingCache::HandleComponentChanged);
	EditingMeshComponent->OnSkeletonChanged().AddUObject(this, &USkeletalMeshEditingCache::HandleSkeletonChanged);

	HostActor->AddInstanceComponent(EditingMeshComponent);
	HostActor->SetRootComponent(EditingMeshComponent);
	HostActor->RegisterAllComponents();

	FTransform ActorTransform = SkeletalMeshComponent->GetComponentTransform();
	
	HostActor->SetActorTransform(ActorTransform);
		
	FTransform PreviewActorTransform = ActorTransform ;
	
	PreviewMesh = NewObject<UPreviewMesh>();
	PreviewMesh->CreateInWorld(World ,PreviewActorTransform);

	UDynamicMesh* EditingDynamicMesh = EditingMeshComponent->GetDynamicMesh();
	EditingDynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
	{
		FDynamicMesh3 PreviewMeshObject = ReadMesh;
		PreviewMesh->ReplaceMesh(MoveTemp(PreviewMeshObject));
	});

	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent()));

	PreviewMesh->SetMaterials(Materials);

	RefSkeletonPoser = NewObject<URefSkeletonPoser>();
	RefSkeletonPoser->SetRefSkeleton(EditingMeshComponent->GetRefSkeleton());
	
	EditingMeshComponent->SetVisibility(false);
	PreviewMesh->SetVisible(false);
	bForceEnableDynamicMesh = false;

	bCacheVisibility = true;
	CacheSkeletonDrawMode = ESkeletonDrawMode::Default;
	bCacheBoneManipulation = true;

	SetSkeletalMeshSkeletonDrawMode(CacheSkeletonDrawMode);

	if (Delegates.OnGetSkeletalMeshSkeletonNotifierDelegate.IsBound())
	{
		TSharedPtr<ISkeletalMeshNotifier> SkeletalMeshSkeletonNotifier = Delegates.OnGetSkeletalMeshSkeletonNotifierDelegate.Execute();
					
		SkeletalMeshSkeletonNotifierBindScope.Reset(
			new FSkeletalMeshNotifierBindScope(GetNotifier(), SkeletalMeshSkeletonNotifier));
	}
	
	SkeletonChangeCountWatcher.Initialize(
		[this]()
			{
				return IsDynamicMeshSkeletonEnabled();
			},
		[this](bool bNewDynamicMeshSkeletonEnabled)
			{
				if (bNewDynamicMeshSkeletonEnabled)
				{
					SkeletalMeshSkeletonNotifierBindScope.Reset();
				}
				else if (Delegates.OnGetSkeletalMeshSkeletonNotifierDelegate.IsBound())
				{
					using namespace UE::SkeletalMeshEditorUtils;
					TSharedPtr<ISkeletalMeshNotifier> SkeletalMeshSkeletonNotifier = Delegates.OnGetSkeletalMeshSkeletonNotifierDelegate.Execute();
					SkeletalMeshSkeletonNotifierBindScope.Reset(
						new FSkeletalMeshNotifierBindScope(GetNotifier(), SkeletalMeshSkeletonNotifier));
					SkeletalMeshSkeletonNotifier->HandleNotification({}, ESkeletalMeshNotifyType::HierarchyChanged);
					SkeletalMeshSkeletonNotifier->HandleNotification(GetSelectedBones(), ESkeletalMeshNotifyType::BonesSelected);
				}
			},
		IsDynamicMeshSkeletonEnabled());

	MeshVisibilityUpdater.Initialize(
		[this]()
			{
				FMeshVisibilityFactors Factors;
				Factors.bIsDynamicMeshEnabled = IsDynamicMeshEnabled();
				Factors.bCacheVisibility = bCacheVisibility;
				return Factors;
			},
		[this](FMeshVisibilityFactors Factors)
			{
				PreviewMesh->SetVisible(
					Factors.bIsDynamicMeshEnabled && Factors.bCacheVisibility);

				if (SkeletalMeshComponent.IsValid())
				{
					SkeletalMeshComponent->SetVisibility(
						!Factors.bIsDynamicMeshEnabled && Factors.bCacheVisibility);
				}

				PreviewMeshVisibilityWatcher.CheckAndUpdate();
			},
		FMeshVisibilityFactors(IsDynamicMeshEnabled(), bCacheVisibility));

	SkeletonVisibilityUpdater.Initialize(
		[this]()
			{
				FSkeletonVisibilityFactors Factors;
				Factors.bIsDynamicMeshSkeletonEnabled = IsDynamicMeshSkeletonEnabled();
				Factors.CacheSkeletonDrawMode = CacheSkeletonDrawMode;
				return Factors;
			},
		[this](FSkeletonVisibilityFactors Factors)
			{
				if (Factors.bIsDynamicMeshSkeletonEnabled)
				{
					SetSkeletalMeshSkeletonDrawMode(ESkeletonDrawMode::Hidden);
				}
				else
				{
					SetSkeletalMeshSkeletonDrawMode(Factors.CacheSkeletonDrawMode);
				}
			},
		FSkeletonVisibilityFactors(IsDynamicMeshSkeletonEnabled(), CacheSkeletonDrawMode)
	);

	BoneManipulationUpdater.Initialize(
		[this]()
			{
				FBoneManipulationFactors Factors;
				Factors.bIsDynamicMeshSkeletonEnabled = IsDynamicMeshSkeletonEnabled();
				Factors.bCacheBoneManipulation = bCacheBoneManipulation;
				return Factors;
			},
		[this](FBoneManipulationFactors Factors)
			{
				Delegates.ToggleSkeletalMeshBoneManipulationDelegate.ExecuteIfBound(
					!Factors.bIsDynamicMeshSkeletonEnabled && bCacheBoneManipulation);
			},
		FBoneManipulationFactors(IsDynamicMeshSkeletonEnabled(), bCacheBoneManipulation)
	);

	PreviewMeshVisibilityWatcher.Initialize(
		[this]()
			{
				return PreviewMesh->IsVisible();
			},
		[this](bool bIsVisible)
			{
				if (bIsVisible)
				{
					RequestDeformPreviewMesh();
				}
			},
		PreviewMesh->IsVisible());

	PoseChangeDetector.GetNotifier().AddUObject(this, &USkeletalMeshEditingCache::HandlePoseChangeDetectorEvent);
}

EMeshLODIdentifier USkeletalMeshEditingCache::GetLOD() const
{
	return EditingMeshComponent->GetLOD();
}


void USkeletalMeshEditingCache::Destroy()
{
	SkeletalMeshSkeletonNotifierBindScope.Reset();
	HostActor->Destroy();
	PreviewMesh->Disconnect();
	
	if (SkeletalMeshComponent.IsValid())
	{
		SkeletalMeshComponent->SetVisibility(true);
		SetSkeletalMeshSkeletonDrawMode(ESkeletonDrawMode::Default);
	}
}

void USkeletalMeshEditingCache::ApplyChanges()
{
	// Convention: callers must guard with HasUnappliedChanges() before invoking. Transaction
	// scope is the caller's responsibility — only UI / top-level APIs open transactions so the
	// undo record gets a descriptive title from the invoking context.
	bool bCommitted = false;
	if (SkeletalMeshComponent.IsValid())
	{
		// See USkeletalMeshComponentToolTarget
		// Unregister the component while we update its skeletal mesh
		FComponentReregisterContext ComponentReregisterContext(SkeletalMeshComponent.Get());

		bCommitted = EditingMeshComponent->CommitToSkeletalMesh();

		// this rebuilds physics, but it doesn't undo!
		SkeletalMeshComponent->RecreatePhysicsState();
	}
	else
	{
		bCommitted = EditingMeshComponent->CommitToSkeletalMesh();
	}

	if (!bCommitted)
	{
		return;
	}

	// Re-apply preview weights against the now-committed morph target names so user intent
	// survives the asset rebuild.
	if (UDebugSkelMeshComponent* Mesh = GetDebugSkelMeshComponent())
	{
		if (GUndo)
		{
			Mesh->SetFlags(RF_Transactional);
			Mesh->Modify();
		}
		constexpr bool bRemoveZeroWeight = false;
		for (const TPair<FName, float>& MorphTargetOverride : MorphTargetOverrides)
		{
			Mesh->SetMorphTarget(MorphTargetOverride.Key, MorphTargetOverride.Value, bRemoveZeroWeight);
		}
	}
}

void USkeletalMeshEditingCache::DiscardChanges()
{
	EditingMeshComponent->DiscardChanges();
}

void USkeletalMeshEditingCache::HandleComponentChanged()
{
	RebuildPreviewMesh();
	MeshVisibilityUpdater.CheckAndUpdate();

	Delegates.OnComponentChangedEvent.Broadcast();
}

void USkeletalMeshEditingCache::HandleSkeletonChanged()
{
	RefSkeletonPoser->SetRefSkeleton(GetEditingMeshComponent()->GetRefSkeleton());
	UpdateSelectedBoneIndices();

	SkeletonChangeCountWatcher.CheckAndUpdate();
	SkeletonVisibilityUpdater.CheckAndUpdate();
	BoneManipulationUpdater.CheckAndUpdate();

	Delegates.OnSkeletonChangedEvent.Broadcast();
}

void USkeletalMeshEditingCache::Tick()
{
	const TArray<FTransform>& BoneTransforms = GetComponentSpaceBoneTransforms();
	const TMap<FName, float>& MorphWeights = GetMorphTargetWeights();
	if (IsDynamicMeshEnabled())
	{
		PoseChangeDetector.CheckPose(BoneTransforms, MorphWeights);
	}

	if (bShouldDeformPreviewMesh)
	{
		bShouldDeformPreviewMesh = false;
		DeformPreviewMesh(BoneTransforms, MorphWeights);
	}
}

bool USkeletalMeshEditingCache::HandleClick(HHitProxy* HitProxy)
{
	if (IsDynamicMeshSkeletonEnabled())
	{
		TArray<FName> SelectedBones;	
		if (const HBoneHitProxy* BoneHitProxy = HitProxyCast<HBoneHitProxy>(HitProxy))
		{
			SelectedBones.Emplace(BoneHitProxy->BoneName);
		}

		Notifier->HandleNotification(SelectedBones, ESkeletalMeshNotifyType::BonesSelected);
	
		return true;
	}

	return false;
}

void USkeletalMeshEditingCache::Render(FPrimitiveDrawInterface* PDI, TFunction<void(FSkelDebugDrawConfig&)> OverrideBoneDrawConfigFunc)
{
	const ESkeletonDrawMode DynamicMeshSkeletonDrawMode =
		IsDynamicMeshSkeletonEnabled() ? CacheSkeletonDrawMode : ESkeletonDrawMode::Hidden;
	if (DynamicMeshSkeletonDrawMode == ESkeletonDrawMode::Hidden)
	{
		return;
	}

	const FReferenceSkeleton& RefSkeleton = EditingMeshComponent->GetRefSkeleton();
	const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
	const FTransform& PreviewActorTransform = PreviewMesh->GetActor()->GetActorTransform();
	
	const int32 NumBones = RefSkeleton.GetRawBoneNum();	

	TArray<FTransform> WorldTransforms;
	WorldTransforms.AddUninitialized(NumBones);

	TArray<FLinearColor> BoneColors;
	BoneColors.AddUninitialized(NumBones);

	TArray<FBoneIndexType> RequiredBones;
	RequiredBones.AddUninitialized(NumBones);
	
	for ( int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex )
	{
		RequiredBones[BoneIndex] = BoneIndex;
		WorldTransforms[BoneIndex] = ComponentSpaceTransforms[BoneIndex] * PreviewActorTransform ;
		BoneColors[BoneIndex] = GetDefaultBoneColor(BoneIndex);
	}

	constexpr bool bForceDraw = false;
	constexpr bool bAddHitProxy = true;
		
	const bool bUseMultiColors = GetDefault<UPersonaOptions>()->bShowBoneColors;
	
	FSkelDebugDrawConfig DrawConfig;
	DrawConfig.BoneDrawMode = EBoneDrawMode::Type::All ;
	DrawConfig.BoneDrawSize = 1.0f;
	DrawConfig.bAddHitProxy = bAddHitProxy;
	DrawConfig.bForceDraw = bForceDraw;
	DrawConfig.bUseMultiColorAsDefaultColor = bUseMultiColors;
	DrawConfig.DefaultBoneColor = GetMutableDefault<UPersonaOptions>()->DefaultBoneColor;
	DrawConfig.AffectedBoneColor = GetMutableDefault<UPersonaOptions>()->AffectedBoneColor;
	DrawConfig.SelectedBoneColor = GetMutableDefault<UPersonaOptions>()->SelectedBoneColor;
	DrawConfig.ParentOfSelectedBoneColor = GetMutableDefault<UPersonaOptions>()->ParentOfSelectedBoneColor;

	OverrideBoneDrawConfigFunc(DrawConfig);

	TArray<TRefCountPtr<HHitProxy>> HitProxies; HitProxies.Reserve(NumBones);

	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		HitProxies.Add(new HBoneHitProxy(Index, RefSkeleton.GetBoneName(Index)));
	}

	SkeletalDebugRendering::DrawBones(
		PDI,
		PreviewActorTransform.GetLocation(),
		RequiredBones,
		RefSkeleton,
		WorldTransforms,
		SelectedBoneIndices,
		BoneColors,
		HitProxies,
		DrawConfig
	);
}

void USkeletalMeshEditingCache::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	if (IsDynamicMeshSkeletonEnabled())
	{
		// See FSkeletonSelectionEditMode::DrawHUD
		const FReferenceSkeleton& RefSkeleton = GetEditingMeshComponent()->GetRefSkeleton();
		const int32 BoneIndex = GetFirstSelectedBoneIndex();

		// Draw name of selected bone
		if (RefSkeleton.IsValidIndex(BoneIndex))
		{
			const FIntPoint ViewPortSize = SkeletalMeshEditingCacheLocal::GetDPIUnscaledSize(Viewport, ViewportClient);
			const int32 HalfX = ViewPortSize.X / 2;
			const int32 HalfY = ViewPortSize.Y / 2;

			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);

			const FMatrix BoneMatrix = (GetComponentSpaceBoneTransforms()[BoneIndex] * GetTransform()).ToMatrixNoScale();
			const FPlane Proj = View->Project(BoneMatrix.GetOrigin());
			if (Proj.W > 0.f)
			{
				const int32 XPos = HalfX + static_cast<int32>(HalfX * Proj.X);
				const int32 YPos = HalfY + static_cast<int32>(HalfY * Proj.Y * -1);

				FCanvasTextItem TextItem(FVector2D(XPos, YPos), FText::FromString(BoneName.ToString()), GEngine->GetSmallFont(), FLinearColor::White);
				TextItem.EnableShadow(FLinearColor::Black);
				Canvas->DrawItem(TextItem);
			}
		}
	}
}

TSharedPtr<ISkeletalMeshNotifier> USkeletalMeshEditingCache::GetNotifier()
{
	if (!Notifier.IsValid())
	{
		Notifier = MakeShared<FSkeletalMeshEditingCacheNotifier>(this);
	}

	return Notifier;
}

void USkeletalMeshEditingCache::HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	switch (InNotifyType)
	{
	case ESkeletalMeshNotifyType::BonesSelected:
		SelectedBoneNames = BoneNames;
		UpdateSelectedBoneIndices();
		break;
	}
}

void USkeletalMeshEditingCache::ToggleForceEnableDynamicMesh(bool bEnable)
{
	if (bForceEnableDynamicMesh == bEnable) 
	{
		return;
	}
	bForceEnableDynamicMesh = bEnable;

	MeshVisibilityUpdater.CheckAndUpdate();
}

USkeletalMeshBackedDynamicMeshComponent* USkeletalMeshEditingCache::GetEditingMeshComponent() const
{
	return EditingMeshComponent;
}

UPrimitiveComponent* USkeletalMeshEditingCache::GetPreviewMeshComponent() const
{
	return PreviewMesh->GetRootComponent();
}


void USkeletalMeshEditingCache::HandleVisibilityChangeRequest(bool bVisible)
{
	if (bCacheVisibility == bVisible)
	{
		return;
	}
	bCacheVisibility = bVisible;

	MeshVisibilityUpdater.CheckAndUpdate();
}



void USkeletalMeshEditingCache::HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload)
{
	using namespace SkeletalMeshToolsHelper;
	
	if (Payload.CurrentState == FPoseChangeDetector::PoseJustChanged ||
		Payload.CurrentState == FPoseChangeDetector::PoseChanged)
	{
		RequestDeformPreviewMesh();
	}
}

const TArray<FTransform>& USkeletalMeshEditingCache::GetComponentSpaceBoneTransforms() const
{
	if (IsDynamicMeshSkeletonEnabled())
	{
		return RefSkeletonPoser->GetComponentSpaceTransforms();
	}

	return SkeletalMeshComponent->GetComponentSpaceTransforms();
}

FTransform USkeletalMeshEditingCache::GetTransform()
{
	return PreviewMesh->GetTransform();
}

void USkeletalMeshEditingCache::ToggleBoneManipulation(bool bEnable)
{
	if (bCacheBoneManipulation == bEnable)
	{
		return;
	}
	bCacheBoneManipulation = bEnable;

	BoneManipulationUpdater.CheckAndUpdate();
}

void USkeletalMeshEditingCache::SetSkeletonDrawMode(ESkeletonDrawMode InSkeletonDrawMode)
{
	if (CacheSkeletonDrawMode == InSkeletonDrawMode)
	{
		return;
	}
	CacheSkeletonDrawMode = InSkeletonDrawMode;

	SkeletonVisibilityUpdater.CheckAndUpdate();
}

bool USkeletalMeshEditingCache::IsDynamicMeshEnabled() const
{
	return bForceEnableDynamicMesh || GetEditingMeshComponent()->GetChangeCount() > 0;
}

bool USkeletalMeshEditingCache::IsDynamicMeshSkeletonEnabled() const
{
	return GetEditingMeshComponent()->GetSkeletonChangeTracker().GetChangeCount() > 0;
}

bool USkeletalMeshEditingCache::IsDynamicMeshBoneManipulationEnabled() const
{
	return IsDynamicMeshSkeletonEnabled() && bCacheBoneManipulation;
}

int32 USkeletalMeshEditingCache::GetFirstSelectedBoneIndex() const
{
	return !SelectedBoneIndices.IsEmpty() ? SelectedBoneIndices[0] : INDEX_NONE;
}

TArray<FName> USkeletalMeshEditingCache::GetSelectedBones() const
{
	return SelectedBoneNames;
}

void USkeletalMeshEditingCache::ResetDynamicMeshBoneTransforms(bool bSelectedOnly)
{
	if (!IsDynamicMeshSkeletonEnabled())
	{
		return;
	}
	
	RefSkeletonPoser->BeginPoseChange();
	if (bSelectedOnly)
	{
		for (int32 BoneIndex : SelectedBoneIndices)
		{
			RefSkeletonPoser->ClearBoneAdditiveTransform(BoneIndex);
		}
	}
	else
	{
		RefSkeletonPoser->ClearAllBoneAdditiveTransforms();
	}
	RefSkeletonPoser->EndPoseChange();
}

TArray<FName> USkeletalMeshEditingCache::GetMorphTargets() const
{
	return EditingMeshComponent->GetMorphTargetChangeTracker().GetCurrentMorphTargetNames();
}

TMap<FName, float> USkeletalMeshEditingCache::GetMorphTargetWeights() const
{
	TMap<FName, float> MorphTargetWeights;

	for (const FName& Name : GetMorphTargets())
	{
		MorphTargetWeights.Emplace(Name, 0.0f);
	}
	
	const TMap<FName, float>& SkeletalMeshWeights = GetSkeletalMeshComponentMorphTargetWeights();

	for (const TPair<FName, float>& SkeletalMeshWeight : SkeletalMeshWeights)
	{
		FName UpdatedName = EditingMeshComponent->GetMorphTargetChangeTracker().GetCurrentMorphTargetName(SkeletalMeshWeight.Key);
		if (UpdatedName != NAME_None)
		{
			MorphTargetWeights[UpdatedName] = SkeletalMeshWeight.Value;
		}
	}

	for (TPair<FName, float>& Entry : MorphTargetWeights)
	{
		if (const float* Override = MorphTargetOverrides.Find(Entry.Key))
		{
			Entry.Value = *Override;
		}
	}

	return MorphTargetWeights;	
}

float USkeletalMeshEditingCache::GetMorphTargetWeight(FName MorphTarget) const
{
	if (const float* Weight = MorphTargetOverrides.Find(MorphTarget))
	{
		return *Weight;
	}

	const TMap<FName, float>& SkeletalMeshWeights = GetSkeletalMeshComponentMorphTargetWeights();

	FName OriginalName = EditingMeshComponent->GetMorphTargetChangeTracker().GetOriginalMorphTargetName(MorphTarget);
	
	if (const float* Weight = SkeletalMeshWeights.Find(OriginalName))
	{
		return *Weight;
	}

	return 0.0f;
}


void USkeletalMeshEditingCache::HandleSetMorphTargetWeight(FName MorphTarget, float Weight)
{
	if (MorphTarget == NAME_None)
	{
		return;
	}

	Modify();
	
	MorphTargetOverrides.Emplace(MorphTarget, Weight);
	
	FName OriginalMorphTargetName = EditingMeshComponent->GetMorphTargetChangeTracker().GetOriginalMorphTargetName(MorphTarget);
	if (OriginalMorphTargetName != NAME_None)
	{
		if (UDebugSkelMeshComponent* Mesh = GetDebugSkelMeshComponent())
		{
			if (GUndo)
			{
				Mesh->SetFlags(RF_Transactional);
				Mesh->Modify();
			}
			constexpr bool bRemoveZeroWeight = false;
			Mesh->SetMorphTarget(OriginalMorphTargetName, Weight, bRemoveZeroWeight);
		}
	}
}

bool USkeletalMeshEditingCache::GetMorphTargetAutoFill(FName Name)
{
	return !MorphTargetOverrides.Contains(Name);
}

void USkeletalMeshEditingCache::HandleSetMorphTargetAutoFill(FName Name, bool bAutoFill, float PreviousOverrideWeight)
{
	Modify();
	
	if (bAutoFill)
	{
		MorphTargetOverrides.Remove(Name);
	}
	else
	{
		MorphTargetOverrides.Emplace(Name, PreviousOverrideWeight);
	}

	if (UDebugSkelMeshComponent* Mesh = GetDebugSkelMeshComponent())
	{
		FName OriginalMorphTargetName = EditingMeshComponent->GetMorphTargetChangeTracker().GetOriginalMorphTargetName(Name);
		if (OriginalMorphTargetName != NAME_None)
		{
			if (GUndo)
			{
				Mesh->SetFlags(RF_Transactional);
				Mesh->Modify();
			}

			if (bAutoFill)
			{
				constexpr bool bRemoveZeroWeight = true;
				Mesh->SetMorphTarget(OriginalMorphTargetName, 0.0f, bRemoveZeroWeight);	
			}
			else
			{
				constexpr bool bRemoveZeroWeight = false;
				Mesh->SetMorphTarget(OriginalMorphTargetName, MorphTargetOverrides[Name], bRemoveZeroWeight);
			}
		}
	}
}

void USkeletalMeshEditingCache::HandleMorphTargetEdited(FName MorphTarget)
{
	EditingMeshComponent->MarkMorphTargetEdited(MorphTarget);	
}

void USkeletalMeshEditingCache::OverrideMorphTargetWeight(FName MorphTarget, float Weight)
{
	HandleSetMorphTargetWeight(MorphTarget, Weight);
}

void USkeletalMeshEditingCache::ClearMorphTargetOverride(FName MorphTarget)
{
	if (MorphTarget != NAME_None)
	{
		constexpr bool bAutoFill = true;
		constexpr float DummyWeight = 0.0f;
		HandleSetMorphTargetAutoFill(MorphTarget, bAutoFill, DummyWeight);
	}
}

FName USkeletalMeshEditingCache::AddMorphTarget(FName InName)
{
	return EditingMeshComponent->AddMorphTarget(InName);
}

TArray<FName> USkeletalMeshEditingCache::AddMorphTargetsIfMissing(const TArray<FName>& Names)
{
	return EditingMeshComponent->AddMorphTargetsIfMissing(Names);
}

FName USkeletalMeshEditingCache::RenameMorphTarget(FName InOldName, FName InNewName)
{
	FName MorphTarget = EditingMeshComponent->RenameMorphTarget(InOldName, InNewName);

	if (float* OverrideWeight = MorphTargetOverrides.Find(InOldName))
	{
		float SavedWeight = *OverrideWeight;
		ClearMorphTargetOverride(InOldName);
		OverrideMorphTargetWeight(InNewName, SavedWeight);
	}

	return MorphTarget;
}

void USkeletalMeshEditingCache::RemoveMorphTargets(const TArray<FName>& InNames)
{
	EditingMeshComponent->RemoveMorphTargets(InNames);

	for (const FName& Name : InNames)
	{
		ClearMorphTargetOverride(Name);
	}
}

TArray<FName> USkeletalMeshEditingCache::DuplicateMorphTargets(const TArray<FName>& InNames)
{
	return EditingMeshComponent->DuplicateMorphTargets(InNames);
}

void USkeletalMeshEditingCache::MirrorMorphTargets(const TArray<FName>& InNames)
{
	EditingMeshComponent->MirrorMorphTargets(InNames);
}

void USkeletalMeshEditingCache::FlipMorphTargets(const TArray<FName>& InNames)
{
	EditingMeshComponent->FlipMorphTargets(InNames);
}

FName USkeletalMeshEditingCache::MergeMorphTargets(const TArray<FName>& InNames)
{
	return EditingMeshComponent->MergeMorphTargets(InNames);
}

void USkeletalMeshEditingCache::ApplyCurrentWeightToMorphTarget(FName InName)
{
	const float CurrentWeight = GetMorphTargetWeight(InName);
	if (FMath::IsNearlyEqual(CurrentWeight, 1.0f))
	{
		return;
	}
	EditingMeshComponent->ApplyWeightToMorphTarget(InName, CurrentWeight);
	OverrideMorphTargetWeight(InName, 1.0f);
}

void USkeletalMeshEditingCache::GenerateFlippedMorphTargets(const TArray<TPair<FName, FName>>& InPairs)
{
	EditingMeshComponent->GenerateFlippedMorphTargets(InPairs);
}

const UE::Geometry::FMeshPlanarSymmetry* USkeletalMeshEditingCache::GetBaseMeshSymmetry()
{
	return EditingMeshComponent ? EditingMeshComponent->GetBaseMeshSymmetry() : nullptr;
}

void USkeletalMeshEditingCache::HandleGeometryUpdate(const FDynamicMesh3& InMesh, FName MorphTargetName)
{
	using namespace SkeletalMeshToolsHelper;
	
	const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
	const TArray<FTransform>& ComponentSpaceTransformsRefPose = GetComponentSpaceBoneTransformsRefPose();

	TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(
			ComponentSpaceTransformsRefPose,
			ComponentSpaceTransforms
			);

	TMap<FName, float> MorphTargetWeights = GetMorphTargetWeights();

	EditingMeshComponent->HandleGeometryUpdate(MorphTargetName, InMesh, BoneMatrices, MorphTargetWeights, GetIsolationSubmesh());
}

URefSkeletonPoser* USkeletalMeshEditingCache::GetSkeletonPoser() const
{
	return RefSkeletonPoser;
}


const TArray<FTransform>& USkeletalMeshEditingCache::GetComponentSpaceBoneTransformsRefPose() const
{
	return EditingMeshComponent->GetComponentSpaceBoneTransformsRefPose();
}

const TMap<FName, float>& USkeletalMeshEditingCache::GetSkeletalMeshComponentMorphTargetWeights() const
{
	UAnimInstance* AnimInstance = SkeletalMeshComponent->GetAnimInstance();
	return AnimInstance->GetAnimationCurveList(EAnimCurveType::MorphTargetCurve);
}

void USkeletalMeshEditingCache::DeformPreviewMesh(const TArray<FTransform>& ComponentSpaceTransforms, const TMap<FName, float>& MorphTargetWeights)
{
	if (!PreviewMesh->IsVisible())
	{
		return;
	}

	if (ComponentSpaceTransforms.IsEmpty())
	{
		return;
	}

	if (GetComponentSpaceBoneTransformsRefPose().Num() != ComponentSpaceTransforms.Num())
	{
		return;
	}
	
	using namespace UE::Geometry;
	using namespace SkeletalMeshToolsHelper;
	
	TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(GetComponentSpaceBoneTransformsRefPose(), ComponentSpaceTransforms);
			
	FDynamicMesh3& EditingMesh = *EditingMeshComponent->GetMesh();

	// When isolated, only pose the vertices in the submesh and remap IDs to preview mesh space
	const FDynamicSubmesh3* Submesh = GetIsolationSubmesh();
	TArray<int32> IsolatedVertArray;
	if (Submesh)
	{
		const int32 NumSubmeshVerts = Submesh->GetSubmesh().VertexCount();
		IsolatedVertArray.Reserve(NumSubmeshVerts);
		for (int32 SubVID : Submesh->GetSubmesh().VertexIndicesItr())
		{
			IsolatedVertArray.Add(Submesh->MapVertexToBaseMesh(SubVID));
		}
	}

	auto DeformPreviewMeshFunc = [&](FDynamicMesh3& VisualMesh)
		{
			auto WriteFunc = [&](FVertInfo VertInfo, const FVector& PosedVertPos)
				{
					int32 PreviewVID = Submesh ? Submesh->MapVertexToSubmesh(VertInfo.VertID) : VertInfo.VertID;
					if (PreviewVID != FDynamicMesh3::InvalidID)
					{
						VisualMesh.SetVertex(PreviewVID, PosedVertPos);
					}
				};
					
			GetPosedMesh(WriteFunc, EditingMesh, BoneMatrices, NAME_None, MorphTargetWeights, IsolatedVertArray);
			FMeshNormals::QuickRecomputeOverlayNormals(VisualMesh);
		};
		
	constexpr bool bRebuildSpatial = false;
	
	// Using UDynamicMeshComponent::EditMesh() to make sure dependent systems, such as seleciton manager, receive a notification.
	// Pass DeformationEdit so the geometry selection manager treats this as vertex-only and skips the FGroupTopology rebuild.
	UDynamicMeshComponent* DynamicMeshComponent = CastChecked<UDynamicMeshComponent>(PreviewMesh->GetRootComponent());
	DynamicMeshComponent->EditMesh(DeformPreviewMeshFunc, EDynamicMeshComponentRenderUpdateMode::NoUpdate,
		EDynamicMeshChangeType::DeformationEdit,
		EDynamicMeshAttributeChangeFlags::VertexPositions | EDynamicMeshAttributeChangeFlags::NormalsTangents);

	Delegates.OnPreviewMeshDeformedEvent.Broadcast();
}

void USkeletalMeshEditingCache::UpdateSelectedBoneIndices()
{
	SelectedBoneIndices.Reset();
	for (const FName Name: SelectedBoneNames)
	{
		SelectedBoneIndices.Add(GetEditingMeshComponent()->GetRefSkeleton().FindRawBoneIndex(Name));
	}
}

void USkeletalMeshEditingCache::SetSkeletalMeshSkeletonDrawMode(ESkeletonDrawMode DrawMode) const
{
	if (UDebugSkelMeshComponent* DebugSkelMeshComponent = GetDebugSkelMeshComponent())
	{
		DebugSkelMeshComponent->SkeletonDrawMode = DrawMode;
	}
}

ESkeletonDrawMode USkeletalMeshEditingCache::GetCurrentSkeletonDrawMode() const
{
	return CacheSkeletonDrawMode;
}

FLinearColor USkeletalMeshEditingCache::GetDefaultBoneColor(int32 InBoneIndex) const
{
	// this returns the normal unmodified color of the bone, calling code must account
	// for any editor specific states that might affect the final bone color (like selection)
	
	// skeleton greyed out
	if (GetCurrentSkeletonDrawMode() == ESkeletonDrawMode::GreyedOut)
	{
		return GetDefault<UPersonaOptions>()->DisabledBoneColor;
	}

	// using default color for all bones
	if (!GetDefault<UPersonaOptions>()->bShowBoneColors)
	{
		return GetDefault<UPersonaOptions>()->DefaultBoneColor;
	}
	
	// uses deterministic, semi-random desaturated color unique to the bone index
	return SkeletalDebugRendering::GetSemiRandomColorForBone(InBoneIndex);	
}

UDebugSkelMeshComponent* USkeletalMeshEditingCache::GetDebugSkelMeshComponent() const
{
	return Cast<UDebugSkelMeshComponent>(SkeletalMeshComponent);
}


// --- Geometry Isolation ---

const TArray<int32>& USkeletalMeshEditingCache::GetIsolatedTriangles() const
{
	return IsolatedTrianglesFromFullMesh;
}

bool USkeletalMeshEditingCache::HasIsolation() const
{
	return !IsolatedTrianglesFromFullMesh.IsEmpty();
}

const UE::Geometry::FDynamicSubmesh3* USkeletalMeshEditingCache::GetIsolationSubmesh() const
{
	return IsolationSubmesh.IsSet() ? &IsolationSubmesh.GetValue() : nullptr;
}

// --- Pure isolation computation ---

bool USkeletalMeshEditingCache::ComputeIsolatedTriangles(
	EIsolationOperation Operation,
	const UE::Geometry::FGeometrySelection& Selection,
	TArray<int32>& OutFullMeshTriangles) const
{
	using namespace UE::Geometry;

	// Selection IDs are in editing-mesh (full mesh) space — enumerate against the editing mesh
	TSet<int32> SelectedTriIDs;
	GetEditingMeshComponent()->ProcessMesh([&](const FDynamicMesh3& EditingMesh)
	{
		EnumerateSelectionTriangles(Selection, EditingMesh,
			[&SelectedTriIDs](int32 TriangleID)
			{
				SelectedTriIDs.Add(TriangleID);
			});
	});

	if (SelectedTriIDs.IsEmpty())
	{
		return false;
	}

	// Triangle IDs are already in full-mesh space — no remapping needed
	OutFullMeshTriangles.Reset();

	if (Operation == EIsolationOperation::Isolate)
	{
		OutFullMeshTriangles = SelectedTriIDs.Array();
	}
	else // Hide: all currently visible triangles that are NOT selected
	{
		if (IsolationSubmesh.IsSet())
		{
			// Currently isolated — hide from the isolated set
			for (int32 TriID : IsolatedTrianglesFromFullMesh)
			{
				if (!SelectedTriIDs.Contains(TriID))
				{
					OutFullMeshTriangles.Add(TriID);
				}
			}
		}
		else
		{
			// Full mesh visible — hide from all triangles
			GetEditingMeshComponent()->ProcessMesh([&](const FDynamicMesh3& EditingMesh)
			{
				for (int32 TriID : EditingMesh.TriangleIndicesItr())
				{
					if (!SelectedTriIDs.Contains(TriID))
					{
						OutFullMeshTriangles.Add(TriID);
					}
				}
			});
		}
	}

	return true;
}

// --- High-level isolation APIs ---

bool USkeletalMeshEditingCache::IsolateSelection(const UE::Geometry::FGeometrySelection& InSelection)
{
	TArray<int32> NewTriangles;
	if (!ComputeIsolatedTriangles(EIsolationOperation::Isolate, InSelection, NewTriangles))
	{
		return false;
	}

	SetTriangleIsolation(MoveTemp(NewTriangles));
	return true;
}

bool USkeletalMeshEditingCache::HideSelection(const UE::Geometry::FGeometrySelection& InSelection)
{
	TArray<int32> NewTriangles;
	if (!ComputeIsolatedTriangles(EIsolationOperation::Hide, InSelection, NewTriangles))
	{
		return false;
	}

	SetTriangleIsolation(MoveTemp(NewTriangles));

	return true;
}

bool USkeletalMeshEditingCache::ShowFullMesh(bool bSaveIsolation)
{
	if (!HasIsolation())
	{
		return false;
	}

	if (bSaveIsolation)
	{
		SavedIsolationTriangles = IsolatedTrianglesFromFullMesh;
	}

	SetTriangleIsolation({});
	return true;
}

bool USkeletalMeshEditingCache::HasSavedIsolation() const
{
	return SavedIsolationTriangles.IsSet();
}

const TArray<int32>& USkeletalMeshEditingCache::GetSavedIsolationTriangles() const
{
	static const TArray<int32> Empty;
	return SavedIsolationTriangles.IsSet() ? SavedIsolationTriangles.GetValue() : Empty;
}

void USkeletalMeshEditingCache::RestoreSavedIsolation()
{
	if (SavedIsolationTriangles.IsSet())
	{
		SetTriangleIsolation(MoveTemp(SavedIsolationTriangles.GetValue()));
		SavedIsolationTriangles.Reset();
	}
}

void USkeletalMeshEditingCache::DiscardSavedIsolation()
{
	SavedIsolationTriangles.Reset();
}

bool USkeletalMeshEditingCache::ConvertIsolationForTopologyMode(UE::Geometry::EGeometryTopologyType NewTopologyType)
{
	using namespace UE::Geometry;

	if (!HasIsolation())
	{
		return false;
	}

	if (NewTopologyType != EGeometryTopologyType::Polygroup)
	{
		// Triangle mode is the finest granularity — no expansion needed
		return false;
	}

	const FGroupTopology& FullMeshTopology = GetEditingMeshComponent()->GetGroupTopology();

	// Collect all group IDs touched by currently isolated triangles
	TSet<int32> TouchedGroupIDs;
	for (int32 TriID : IsolatedTrianglesFromFullMesh)
	{
		TouchedGroupIDs.Add(FullMeshTopology.GetGroupID(TriID));
	}

	// Expand to all triangles in those groups
	TArray<int32> ExpandedTriangles;
	for (int32 GroupID : TouchedGroupIDs)
	{
		ExpandedTriangles.Append(FullMeshTopology.GetGroupTriangles(GroupID));
	}

	if (ExpandedTriangles.Num() == IsolatedTrianglesFromFullMesh.Num())
	{
		// Already aligned with polygroup boundaries — nothing to do
		return false;
	}

	SetTriangleIsolation(MoveTemp(ExpandedTriangles));
	return true;
}

// --- Isolation state mutation ---

namespace
{
	class FGeometryIsolationChange : public FToolCommandChange
	{
	public:
		TArray<int32> TrianglesBefore;
		TArray<int32> TrianglesAfter;

		virtual void Apply(UObject* Object) override
		{
			if (USkeletalMeshEditingCache* Cache = Cast<USkeletalMeshEditingCache>(Object))
			{
				Cache->SetTriangleIsolation(TrianglesAfter);
			}
		}

		virtual void Revert(UObject* Object) override
		{
			if (USkeletalMeshEditingCache* Cache = Cast<USkeletalMeshEditingCache>(Object))
			{
				Cache->SetTriangleIsolation(TrianglesBefore);
			}
		}

		virtual FString ToString() const override
		{
			return TEXT("Geometry Isolation Change");
		}
	};
}

void USkeletalMeshEditingCache::SetTriangleIsolation(TArray<int32> InFullMeshTriangles)
{
	using namespace UE::Geometry;

	TArray<int32> OldTriangles = MoveTemp(IsolatedTrianglesFromFullMesh);

	// Build new submesh from the incoming triangles
	IsolatedTrianglesFromFullMesh = MoveTemp(InFullMeshTriangles);
	IsolationSubmesh.Reset();
	if (!IsolatedTrianglesFromFullMesh.IsEmpty())
	{
		GetEditingMeshComponent()->ProcessMesh([this](const FDynamicMesh3& FullMesh)
		{
			IsolationSubmesh = FDynamicSubmesh3(&FullMesh, IsolatedTrianglesFromFullMesh);
		});
	}

	RebuildPreviewMesh();
	OnIsolationChanged.Broadcast();

	// Record undo
	if (TransactionsAPI && !GIsTransacting)
	{
		TUniquePtr<FGeometryIsolationChange> Change = MakeUnique<FGeometryIsolationChange>();
		Change->TrianglesBefore = MoveTemp(OldTriangles);
		Change->TrianglesAfter = IsolatedTrianglesFromFullMesh;
		TransactionsAPI->AppendChange(this, MoveTemp(Change), LOCTEXT("IsolationChange", "Isolation Change"));
	}
}

void USkeletalMeshEditingCache::RequestDeformPreviewMesh()
{
	bShouldDeformPreviewMesh = true;
}

void USkeletalMeshEditingCache::RebuildPreviewMesh()
{
	using namespace UE::Geometry;

	GetEditingMeshComponent()->ProcessMesh([&](const FDynamicMesh3& FullMesh)
	{
			const FDynamicMesh3* NewPreviewMesh = IsolationSubmesh ? &IsolationSubmesh->GetSubmesh() : &FullMesh;
			PreviewMesh->ReplaceMesh(*NewPreviewMesh);
	});

	RequestDeformPreviewMesh();
}


#undef LOCTEXT_NAMESPACE
