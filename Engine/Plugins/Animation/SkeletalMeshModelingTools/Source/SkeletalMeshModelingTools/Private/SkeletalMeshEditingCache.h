// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SkeletalMeshNotifier.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Changes/ValueWatcher.h"
#include "DynamicSubmesh3.h"
#include "GroupTopology.h"
#include "Selections/GeometrySelection.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include "SkeletalMeshEditingCache.generated.h"

#define UE_API SKELETALMESHMODELINGTOOLS_API

class AActor;
class USkeletalMeshBackedDynamicMeshComponent;
class UPreviewMesh;
class USkeletalMeshComponent;
class URefSkeletonPoser;
class FPrimitiveDrawInterface;
struct FSkelDebugDrawConfig;
class FEditorViewportClient;
class FViewport;
class FSceneView;
class FCanvas;
class IToolsContextTransactionsAPI;

/**
 * Transient session settings for the morph-target batch operations exposed by SMorphTargetManager.
 * Values live on the CDO and reset when the editor closes — no disk persistence.
 *
 * NOTE: Co-located with USkeletalMeshEditingCache for now; a follow-up refactor will move it.
 */
UCLASS()
class UMorphTargetNamingConventionSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Wildcard pattern that identifies left-side morph targets. Must contain exactly one '*'. */
	UPROPERTY(EditAnywhere, Category = "Naming Convention")
	FString LeftPattern = TEXT("*_l");

	/** Wildcard pattern for the right-side counterpart. Must contain exactly one '*'. The '*' is replaced with the stem extracted from the matching left morph. */
	UPROPERTY(EditAnywhere, Category = "Naming Convention")
	FString RightPattern = TEXT("*_r");
};


class USkeletalMeshEditingCache;
class FSkeletalMeshEditingCacheNotifier: public ISkeletalMeshNotifier
{
public:
	FSkeletalMeshEditingCacheNotifier(USkeletalMeshEditingCache* InEditingCahe);
	virtual void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) override;
private:
	TWeakObjectPtr<USkeletalMeshEditingCache> EditingCache;
};


UCLASS(MinimalAPI, Transient)
class USkeletalMeshEditingCache: public UObject
{
	GENERATED_BODY()
	
public:
	DECLARE_DELEGATE_OneParam(FToggleSkeletalMeshBoneManipulation, bool);
	DECLARE_DELEGATE_RetVal(bool, FIsSkeletalMeshBoneManipulationEnabled);
	DECLARE_DELEGATE_RetVal(TSharedPtr<ISkeletalMeshNotifier>, FOnGetSkeletalMeshSkeletonNotifier);
	struct FDelegates
	{
		FToggleSkeletalMeshBoneManipulation ToggleSkeletalMeshBoneManipulationDelegate;
		FIsSkeletalMeshBoneManipulationEnabled IsSkeletalMeshBoneManipulationEnabledDelegate;
		FOnGetSkeletalMeshSkeletonNotifier OnGetSkeletalMeshSkeletonNotifierDelegate;
		
		FSimpleMulticastDelegate OnComponentChangedEvent;
		FSimpleMulticastDelegate OnSkeletonChangedEvent;
		FSimpleMulticastDelegate OnPreviewMeshDeformedEvent;
	};

	void Spawn(UWorld* World, USkeletalMeshComponent* SkeletalMeshComponent, EMeshLODIdentifier LOD, const FDelegates& InDelegates, IToolsContextTransactionsAPI* InTransactionsAPI);
	EMeshLODIdentifier GetLOD() const;

	void Destroy();

	void ApplyChanges();

	void DiscardChanges();

	void HandleComponentChanged();
	void HandleSkeletonChanged();
	
	void Tick();
	bool HandleClick(HHitProxy *HitProxy);
	void Render(FPrimitiveDrawInterface* PDI, TFunction<void(FSkelDebugDrawConfig&)> OverrideBoneDrawConfigFunc);
	void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas);

	TSharedPtr<ISkeletalMeshNotifier> GetNotifier();
	void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType);

	void ToggleForceEnableDynamicMesh(bool bEnable);
	
	USkeletalMeshBackedDynamicMeshComponent* GetEditingMeshComponent() const;
	UPrimitiveComponent* GetPreviewMeshComponent() const;
	const TArray<FTransform>& GetComponentSpaceBoneTransforms() const;
	FTransform GetTransform();

	void ToggleBoneManipulation(bool bEnable);
	void SetSkeletonDrawMode(ESkeletonDrawMode InSkeletonDrawMode);
	
	bool IsDynamicMeshSkeletonEnabled() const;
	bool IsDynamicMeshBoneManipulationEnabled() const;
	int32 GetFirstSelectedBoneIndex() const;
	TArray<FName> GetSelectedBones() const;
	void ResetDynamicMeshBoneTransforms(bool bSelectedOnly);
	
	TArray<FName> GetMorphTargets() const;
	TMap<FName, float> GetMorphTargetWeights() const;
	float GetMorphTargetWeight(FName MorphTarget) const;

	void HandleSetMorphTargetWeight(FName MorphTarget, float Weight);
	bool GetMorphTargetAutoFill(FName MorphTarget);
	void HandleSetMorphTargetAutoFill(FName MorphTarget, bool bAutoFill, float PreviousOverrideWeight);
	void HandleMorphTargetEdited(FName MorphTarget);

	void OverrideMorphTargetWeight(FName MorphTarget, float Weight);
	void ClearMorphTargetOverride(FName MorphTarget);

	FName AddMorphTarget(FName InName);
	TArray<FName> AddMorphTargetsIfMissing(const TArray<FName>& Names);
	FName RenameMorphTarget(FName InOldName, FName InNewName);
	void RemoveMorphTargets(const TArray<FName>& InNames);
	TArray<FName> DuplicateMorphTargets(const TArray<FName>& InNames);
	void MirrorMorphTargets(const TArray<FName>& InNames);
	void FlipMorphTargets(const TArray<FName>& InNames);
	FName MergeMorphTargets(const TArray<FName>& InNames);
	void  ApplyCurrentWeightToMorphTarget(FName InName);
	void  GenerateFlippedMorphTargets(const TArray<TPair<FName, FName>>& InPairs);

	void HandleGeometryUpdate(const UE::Geometry::FDynamicMesh3& InMesh, FName MorphTargetName);

	const UE::Geometry::FMeshPlanarSymmetry* GetBaseMeshSymmetry();
	
	URefSkeletonPoser* GetSkeletonPoser() const;

	// Geometry isolation — high-level APIs that compute 
	// new isolation, mutate state, rebuild preview mesh
	// Returns false if no-op.
	bool IsolateSelection(const UE::Geometry::FGeometrySelection& InSelection);
	bool HideSelection(const UE::Geometry::FGeometrySelection& InSelection);
	bool ShowFullMesh(bool bSaveIsolation = false);
	
	// Converts isolation triangles when switching topology modes (e.g. expands to full polygroups).
	// Returns false if no-op. 
	bool ConvertIsolationForTopologyMode(UE::Geometry::EGeometryTopologyType NewTopologyType);

	bool HasIsolation() const;
	const TArray<int32>& GetIsolatedTriangles() const;
	const UE::Geometry::FDynamicSubmesh3* GetIsolationSubmesh() const;

	FSimpleMulticastDelegate& GetOnIsolationChanged() { return OnIsolationChanged; }

	// Saved isolation — allows temporarily showing full mesh and restoring later (e.g. during tool use)
	bool HasSavedIsolation() const;
	const TArray<int32>& GetSavedIsolationTriangles() const;
	void RestoreSavedIsolation();
	void DiscardSavedIsolation();


	// Sets isolation state and rebuilds the preview mesh. Records undo via TransactionsAPI.
	void SetTriangleIsolation(TArray<int32> InFullMeshTriangles);
	
	void RequestDeformPreviewMesh();
protected:
	UPROPERTY()
	TObjectPtr<AActor> HostActor;

	UPROPERTY()
	TObjectPtr<USkeletalMeshBackedDynamicMeshComponent> EditingMeshComponent;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	FDelegates Delegates;
	IToolsContextTransactionsAPI* TransactionsAPI = nullptr;

	bool bShouldDeformPreviewMesh = false;
	
	// Flags that drives both dyna mesh and skeletal mesh
	bool bCacheVisibility = true;
	ESkeletonDrawMode CacheSkeletonDrawMode = ESkeletonDrawMode::Default;
	bool bCacheBoneManipulation = true;

	bool bForceEnableDynamicMesh = false;

	bool IsDynamicMeshEnabled() const;

	// Rebinds the skeletal-mesh notifier scope when IsDynamicMeshSkeletonEnabled() transitions.
	TValueWatcher<bool> SkeletonChangeCountWatcher;

	struct FMeshVisibilityFactors
	{
		bool bIsDynamicMeshEnabled= false;
		bool bCacheVisibility = true;
		friend bool operator==(const FMeshVisibilityFactors&, const FMeshVisibilityFactors&) = default;
	};

	TValueWatcher<FMeshVisibilityFactors> MeshVisibilityUpdater;


	struct FSkeletonVisibilityFactors
	{
		bool bIsDynamicMeshSkeletonEnabled = false;
		ESkeletonDrawMode CacheSkeletonDrawMode = ESkeletonDrawMode::Default;
		friend bool operator==(const FSkeletonVisibilityFactors&, const FSkeletonVisibilityFactors&) = default;
	};
	
	TValueWatcher<FSkeletonVisibilityFactors> SkeletonVisibilityUpdater;


	struct FBoneManipulationFactors
	{
		bool bIsDynamicMeshSkeletonEnabled = false;
		bool bCacheBoneManipulation = true;
		friend bool operator==(const FBoneManipulationFactors&, const FBoneManipulationFactors&) = default;
	};
	
	TValueWatcher<FBoneManipulationFactors> BoneManipulationUpdater;
	
	TValueWatcher<bool> PreviewMeshVisibilityWatcher;

	void HandleVisibilityChangeRequest(bool bVisible);

	UPROPERTY()
	TObjectPtr<URefSkeletonPoser> RefSkeletonPoser;

	UPROPERTY()
	TMap<FName, float> MorphTargetOverrides;

	SkeletalMeshToolsHelper::FPoseChangeDetector PoseChangeDetector;
	
	void HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload);

	const TArray<FTransform>& GetComponentSpaceBoneTransformsRefPose() const;

	const TMap<FName, float>& GetSkeletalMeshComponentMorphTargetWeights() const;

	void DeformPreviewMesh(const TArray<FTransform>& ComponentSpaceTransforms, const TMap<FName, float>& MorphTargetWeights);
	
	TSharedPtr<FSkeletalMeshEditingCacheNotifier> Notifier;
	TUniquePtr<FSkeletalMeshNotifierBindScope> SkeletalMeshSkeletonNotifierBindScope;
	
	TArray<FName> SelectedBoneNames;
	TArray<int32> SelectedBoneIndices;
	void UpdateSelectedBoneIndices();

	void SetSkeletalMeshSkeletonDrawMode(ESkeletonDrawMode DrawMode) const;
	ESkeletonDrawMode GetCurrentSkeletonDrawMode() const;
	
	FLinearColor GetDefaultBoneColor(int32 BoneIndex) const;

	UDebugSkelMeshComponent* GetDebugSkelMeshComponent() const;

	// Geometry isolation state
	TArray<int32> IsolatedTrianglesFromFullMesh;
	TOptional<UE::Geometry::FDynamicSubmesh3> IsolationSubmesh;
	TOptional<TArray<int32>> SavedIsolationTriangles;
	FSimpleMulticastDelegate OnIsolationChanged;

	void RebuildPreviewMesh();

	// Pure computation: given an operation and current selection, computes the new isolation
	// triangle set (in full mesh space)
	enum class EIsolationOperation
	{
		Isolate, 
		Hide
	};
	
	bool ComputeIsolatedTriangles(
		EIsolationOperation Operation,
		const UE::Geometry::FGeometrySelection& Selection,
		TArray<int32>& OutFullMeshTriangles) const;

};


#undef UE_API
