// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshEditorUtils.h"

#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "InteractiveToolsContext.h"
#include "ISkeletonEditorModule.h"
#include "SkeletalMeshEditingCache.h"
#include "SkeletalMeshModelingToolsEditorMode.h"
#include "SkeletonModifier.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshEditorUtils)

bool UE::SkeletalMeshEditorUtils::RegisterEditorContextObject(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		const USkeletalMeshEditorContextObject* Found = ToolsContext->ContextObjectStore->FindContext<USkeletalMeshEditorContextObject>();
		if (Found)
		{
			return true;
		}
		
		USkeletalMeshEditorContextObject* ContextObject = NewObject<USkeletalMeshEditorContextObject>(ToolsContext->ToolManager);
		if (ensure(ContextObject))
		{
			ContextObject->Register(ToolsContext->ToolManager);
			return true;
		}
	}
	return false;
}

bool UE::SkeletalMeshEditorUtils::UnregisterEditorContextObject(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		USkeletalMeshEditorContextObject* Found = ToolsContext->ContextObjectStore->FindContext<USkeletalMeshEditorContextObject>();
		if (Found != nullptr)
		{
			Found->Unregister(ToolsContext->ToolManager);
			ToolsContext->ContextObjectStore->RemoveContextObject(Found);
		}
		return true;
	}
	return false;
}

USkeletalMeshEditorContextObject* UE::SkeletalMeshEditorUtils::GetEditorContextObject(UInteractiveToolsContext* ToolsContext)
{
	return ToolsContext->ContextObjectStore->FindContext<USkeletalMeshEditorContextObject>();
}


void USkeletalMeshEditorContextObject::Register(UInteractiveToolManager* InToolManager)
{
	if (ensure(!bRegistered) == false)
	{
		return;
	}

	InToolManager->GetContextObjectStore()->AddContextObject(this);
	bRegistered = true;
}

void USkeletalMeshEditorContextObject::Unregister(UInteractiveToolManager* InToolManager)
{
	ensure(bRegistered);
	
	InToolManager->GetContextObjectStore()->RemoveContextObject(this);

	EditorBindings.Reset();
	
	bRegistered = false;
}

void USkeletalMeshEditorContextObject::Init(USkeletalMeshModelingToolsEditorMode* InEditorMode)
{
	EditorMode = InEditorMode;
	
	EditorBindings.Reset();
}

EMeshLODIdentifier USkeletalMeshEditorContextObject::GetEditingLOD()
{
	if (!EditorMode.IsValid())
	{
		return EMeshLODIdentifier::Default;
	}
	return EditorMode->GetEditingLOD();
}

void USkeletalMeshEditorContextObject::HideSkeleton()
{
	if (!EditorMode.IsValid())
	{
		return;
	}
	EditorMode->HideSkeletonForTool();
}

void USkeletalMeshEditorContextObject::ShowSkeleton()
{
	if (!EditorMode.IsValid())
	{
		return;
	}
	EditorMode->ShowSkeletonForTool();
}

void USkeletalMeshEditorContextObject::ToggleBoneManipulation(bool bEnable)
{
	if (!EditorMode.IsValid())
	{
		return;
	}
	EditorMode->ToggleBoneManipulation(bEnable);
}

const TArray<FTransform>& USkeletalMeshEditorContextObject::GetComponentSpaceBoneTransforms(UToolTarget* InToolTarget)
{
	return EditorMode->GetCurrentEditingCache()->GetComponentSpaceBoneTransforms();
}


FName USkeletalMeshEditorContextObject::GetEditingMorphTarget()
{
	return EditorMode->GetEditingMorphTarget();
}

TMap<FName, float> USkeletalMeshEditorContextObject::GetMorphTargetWeights()
{
	return EditorMode->GetCurrentEditingCache()->GetMorphTargetWeights();
}

void USkeletalMeshEditorContextObject::NotifyMorphTargetEdited()
{
	return EditorMode->GetCurrentEditingCache()->HandleMorphTargetEdited( EditorMode->GetEditingMorphTarget() );
}

const UE::Geometry::FMeshPlanarSymmetry* USkeletalMeshEditorContextObject::GetBaseMeshSymmetry()
{
	if (!EditorMode.IsValid())
	{
		return nullptr;
	}
	USkeletalMeshEditingCache* Cache = EditorMode->GetCurrentEditingCache();
	return Cache ? Cache->GetBaseMeshSymmetry() : nullptr;
}

const TArray<int32>& USkeletalMeshEditorContextObject::GetIsolatedTriangles() const
{
	static const TArray<int32> Empty;
	return (EditorMode.IsValid() && EditorMode->GetCurrentEditingCache()) ? EditorMode->GetCurrentEditingCache()->GetSavedIsolationTriangles() : Empty;
}

void USkeletalMeshEditorContextObject::BindTo(ISkeletalMeshEditingInterface* InEditingInterface)
{
	if (!InEditingInterface)
	{
		return;
	}
	
	BindEditor(InEditingInterface);
	EditorMode->BindToolSkeletonTree(InEditingInterface);
}

void USkeletalMeshEditorContextObject::UnbindFrom(ISkeletalMeshEditingInterface* InEditingInterface)
{
	if (!InEditingInterface)
	{
		return;
	}

	if (EditorMode.IsValid())
	{
		EditorMode->UnbindToolSkeletonTree();
	}
	UnbindEditor(InEditingInterface);
}



USkeletalMeshEditorContextObject::FBindData USkeletalMeshEditorContextObject::BindInterfaceTo(
	ISkeletalMeshEditingInterface* InInterface,
	TSharedPtr<ISkeletalMeshNotifier> InOtherNotifier)
{
	FBindData BindData;
	BindData.BindScope.Reset(new FSkeletalMeshNotifierBindScope(InInterface->GetNotifier(), InOtherNotifier));
	
	return BindData;

}

void USkeletalMeshEditorContextObject::UnbindInterfaceFrom(
	FBindData& InOutBindData)
{
	InOutBindData.BindScope.Reset();
}

void USkeletalMeshEditorContextObject::BindEditor(ISkeletalMeshEditingInterface* InEditingInterface)
{
	if (!InEditingInterface || !EditorMode.IsValid())
	{
		return;
	}
	
	if (EditorBindings.Contains(InEditingInterface))
	{
		return;
	}

	TSharedPtr<ISkeletalMeshEditorBinding> Binding = EditorMode.Pin()->GetModeBinding();
	if (!Binding.IsValid())
	{
		return;
	}
	
	EditorBindings.Emplace(InEditingInterface, BindInterfaceTo(InEditingInterface, Binding->GetNotifier()));

	InEditingInterface->GetNotifier()->HandleNotification(Binding->GetSelectedBones(), ESkeletalMeshNotifyType::BonesSelected);
}

void USkeletalMeshEditorContextObject::UnbindEditor(ISkeletalMeshEditingInterface* InEditingInterface)
{
	if (FBindData* BindData = EditorBindings.Find(InEditingInterface))
	{
		BindData->BindScope.Reset();
	
		EditorBindings.Remove(InEditingInterface);
	}
}

TSharedPtr<ISkeletalMeshEditorBinding> USkeletalMeshEditorContextObject::GetBinding() const
{
	return EditorMode.IsValid() ? EditorMode->GetModeBinding() : nullptr;
}
