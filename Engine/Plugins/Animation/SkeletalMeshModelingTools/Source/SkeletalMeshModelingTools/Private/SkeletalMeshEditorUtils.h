// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/DebugSkelMeshComponent.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"

#include "UObject/Object.h"
#include "UObject/WeakInterfacePtr.h"

#include "SkeletalMeshEditorUtils.generated.h"

#define UE_API SKELETALMESHMODELINGTOOLS_API

class USkeletalMeshModelingToolsEditorMode;
class UInteractiveToolsContext;
class UInteractiveToolManager;
class USkeletalMeshEditorContextObject;
class SReferenceSkeletonTree;

namespace UE
{
	
namespace SkeletalMeshEditorUtils
{
	UE_API bool RegisterEditorContextObject(UInteractiveToolsContext* InToolsContext);
	UE_API bool UnregisterEditorContextObject(UInteractiveToolsContext* InToolsContext);

	UE_API USkeletalMeshEditorContextObject* GetEditorContextObject(UInteractiveToolsContext* InToolsContext);

}
	
}

/**
 * USkeletalMeshEditorContextObject
 */

UCLASS(MinimalAPI)
class USkeletalMeshEditorContextObject : public USkeletalMeshEditorContextObjectBase
{
	GENERATED_BODY()

public:
	UE_API void Register(UInteractiveToolManager* InToolManager);
	UE_API void Unregister(UInteractiveToolManager* InToolManager);

	UE_API void Init(USkeletalMeshModelingToolsEditorMode* InEditorMode);

	UE_API virtual EMeshLODIdentifier GetEditingLOD() override;
	
	UE_API virtual void HideSkeleton() override;
	UE_API virtual void ShowSkeleton() override;
	UE_API virtual void ToggleBoneManipulation(bool bEnable) override;
	UE_API virtual const TArray<FTransform>& GetComponentSpaceBoneTransforms(UToolTarget* InToolTarget) override;

	UE_API virtual FName GetEditingMorphTarget() override;
	UE_API virtual TMap<FName, float> GetMorphTargetWeights() override;
	UE_API virtual void NotifyMorphTargetEdited() override;
	UE_API virtual const UE::Geometry::FMeshPlanarSymmetry* GetBaseMeshSymmetry() override;
	UE_API virtual const TArray<int32>& GetIsolatedTriangles() const override;
	

	UE_API virtual void BindTo(ISkeletalMeshEditingInterface* InEditingInterface) override;
	UE_API virtual void UnbindFrom(ISkeletalMeshEditingInterface* InEditingInterface) override;
	
private:
	TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> EditorMode = nullptr;

	UE_API TSharedPtr<ISkeletalMeshEditorBinding> GetBinding() const;

	struct FBindData
	{
		TUniquePtr<FSkeletalMeshNotifierBindScope> BindScope;
	};
	TMap< ISkeletalMeshEditingInterface*, FBindData > EditorBindings;

	static UE_API FBindData BindInterfaceTo(
		ISkeletalMeshEditingInterface* InInterface,
		TSharedPtr<ISkeletalMeshNotifier> InOtherNotifier);
	
	static UE_API void UnbindInterfaceFrom(
		FBindData& InOutBindData);

	UE_API void BindEditor(ISkeletalMeshEditingInterface* InEditingInterface);
	UE_API void UnbindEditor(ISkeletalMeshEditingInterface* InEditingInterface);

	
	bool bRegistered = false;
};

#undef UE_API
