// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshGizmoUtils.h"

#include "ContextObjectStore.h"
#include "InteractiveToolsContext.h"
#include "BaseGizmos/GizmoElementHitTargets.h"
#include "SkeletalMesh/SkeletonTransformProxy.h"

#include "EditorGizmos/TransformGizmo.h"
#include "Editor/Experimental/EditorInteractiveToolsFramework/Public/EditorInteractiveGizmoManager.h"
#include "EditorGizmos/EditorTransformGizmoBuilder.h"

#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshGizmoUtils)

bool UE::SkeletalMeshGizmoUtils::RegisterTransformGizmoContextObject(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		const USkeletalMeshGizmoContextObject* Found = ToolsContext->ContextObjectStore->FindContext<USkeletalMeshGizmoContextObject>();
		if (Found)
		{
			return true;
		}
		
		USkeletalMeshGizmoContextObject* GizmoContextObject = NewObject<USkeletalMeshGizmoContextObject>(ToolsContext->ToolManager);
		if (ensure(GizmoContextObject))
		{
			GizmoContextObject->RegisterGizmosWithManager(ToolsContext->ToolManager);
			return true;
		}
	}
	return false;
}

bool UE::SkeletalMeshGizmoUtils::UnregisterTransformGizmoContextObject(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		USkeletalMeshGizmoContextObject* Found = ToolsContext->ContextObjectStore->FindContext<USkeletalMeshGizmoContextObject>();
		if (Found != nullptr)
		{
			Found->UnregisterGizmosWithManager(ToolsContext->ToolManager);
			ToolsContext->ContextObjectStore->RemoveContextObject(Found);
		}
		return true;
	}
	return false;
}

UTransformGizmo* UE::SkeletalMeshGizmoUtils::CreateTransformGizmo(UInteractiveToolManager* ToolManager, void* Owner)
{
	if (!ToolManager)
	{
		return nullptr;
	}
	
	UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(ToolManager->GetPairedGizmoManager());
	if (!GizmoManager)
	{
		return nullptr;
	}

	const USkeletalMeshGizmoContextObject* Found = GizmoManager->GetContextObjectStore()->FindContext<USkeletalMeshGizmoContextObject>();
	if (!Found)
	{
		return nullptr;
	}

	static const FString InstanceIdentifier;
	UTransformGizmo* NewGizmo = Cast<UTransformGizmo>( GizmoManager->CreateGizmo(
		USkeletalMeshGizmoContextObject::TransformBuilderIdentifier(), InstanceIdentifier, Owner) );

	if (ensure(NewGizmo))
	{
		GizmoManager->OnGizmosParametersChangedDelegate().AddUObject(NewGizmo, &UTransformGizmo::OnParametersChanged);
		if (GizmoManager->GetDefaultGizmosParameters())
		{
			NewGizmo->OnParametersChanged(*GizmoManager->GetDefaultGizmosParameters());
		}
	}
	
	return NewGizmo;
}

const FString& USkeletalMeshGizmoContextObject::TransformBuilderIdentifier()
{
	static const FString Identifier(TEXT("SkeletalMeshTransformGizmo"));
	return Identifier;
}

void USkeletalMeshGizmoContextObject::RegisterGizmosWithManager(UInteractiveToolManager* InToolManager)
{
	if (ensure(!bGizmosRegistered) == false)
	{
		return;
	}

	InToolManager->GetContextObjectStore()->AddContextObject(this);
	
	UEditorInteractiveGizmoManager* GizmoManager = Cast<UEditorInteractiveGizmoManager>(InToolManager->GetPairedGizmoManager());
	if (ensure(GizmoManager))
	{
		UEditorTransformGizmoBuilder* GizmoBuilder = NewObject<UEditorTransformGizmoBuilder>(this);
		GizmoBuilder->CustomizationFunction = [] { return GetGizmoCustomization(); };
		GizmoManager->RegisterGizmoType(TransformBuilderIdentifier(), GizmoBuilder);
	}
	
	bGizmosRegistered = true;
}

void USkeletalMeshGizmoContextObject::UnregisterGizmosWithManager(UInteractiveToolManager* InToolManager)
{
	ensure(bGizmosRegistered);
	
	InToolManager->GetContextObjectStore()->RemoveContextObject(this);

	UInteractiveGizmoManager* GizmoManager = InToolManager->GetPairedGizmoManager();
	GizmoManager->DeregisterGizmoType(TransformBuilderIdentifier());
	
	bGizmosRegistered = false;
}

USkeletalMeshGizmoWrapperBase* USkeletalMeshGizmoContextObject::GetNewWrapper(UInteractiveToolManager* InToolManager, UObject* Outer, IGizmoStateTarget* InStateTarget, const TScriptInterface<ITransformGizmoSource>& GizmoSource)
{
	if (!bGizmosRegistered)
	{
		return nullptr;
	}
	
	UTransformGizmo* Gizmo = UE::SkeletalMeshGizmoUtils::CreateTransformGizmo(InToolManager, Outer);
	if (!Gizmo)
	{
		return nullptr;
	}
	
	Gizmo->TransformGizmoSource = GizmoSource;
	Gizmo->SetVisibility(true);

	USkeletonTransformProxy* SkeletonTransformProxy = NewObject<USkeletonTransformProxy>(Outer);
	SkeletonTransformProxy->bRotatePerObject = true;
	
	Gizmo->SetActiveTarget(SkeletonTransformProxy, nullptr, InStateTarget);

	if (UGizmoElementHitMultiTarget* HitMultiTarget = Cast< UGizmoElementHitMultiTarget>(Gizmo->HitTarget))
	{
		HitMultiTarget->GizmoTransformProxy = SkeletonTransformProxy;
	}

	USkeletalMeshGizmoWrapper* GizmoWrapper = NewObject<USkeletalMeshGizmoWrapper>(Outer);
	GizmoWrapper->TransformGizmo = Gizmo;
	GizmoWrapper->TransformProxy = SkeletonTransformProxy;

	if (Gizmo->HitTarget)
	{
		TWeakObjectPtr<USkeletalMeshGizmoWrapper> WeakWrapper = MakeWeakObjectPtr(GizmoWrapper);
		Gizmo->HitTarget->Condition = [WeakWrapper](const FInputDeviceRay&)
		{
			return WeakWrapper.IsValid() ? WeakWrapper->CanInteract() : false;
		};
	}
	
	return GizmoWrapper;
}

const FGizmoCustomization& USkeletalMeshGizmoContextObject::GetGizmoCustomization()
{
	static FGizmoCustomization Customization;
		Customization.SizeCoefficient = 1.5f;

	return Customization;
}

bool USkeletalMeshGizmoWrapper::CanInteract() const
{
	if (!TransformProxy || !TransformGizmo)
	{
		return false;
	}
	return TransformProxy->IsValid() && TransformGizmo->bVisible; 
}

bool USkeletalMeshGizmoWrapper::IsGizmoHit(const FInputDeviceRay& PressPos) const
{
	if (!TransformGizmo || !TransformGizmo->bVisible)
	{
		return false;
	}
	return TransformGizmo->HitTarget ? TransformGizmo->HitTarget->IsHit(PressPos).bHit : false;
}

void USkeletalMeshGizmoWrapper::Clear()
{
	if (TransformProxy)
	{
		TransformProxy->Initialize(FTransform::Identity, EToolContextCoordinateSystem::Local);
	}
	
	if (TransformGizmo)
	{
		TransformGizmo->SetVisibility(false);
		TransformGizmo->ClearActiveTarget();

		if (UInteractiveGizmoManager* GizmoManager = TransformGizmo->GetGizmoManager())
		{
			GizmoManager->DestroyGizmo(TransformGizmo);
		}
	}
}

void USkeletalMeshGizmoWrapper::Initialize(const FTransform& InTransform, const EToolContextCoordinateSystem& InTransformMode)
{
	if (TransformProxy)
	{
		TransformProxy->Initialize(InTransform, InTransformMode);
	}
	
	if (TransformGizmo)
	{
		TransformGizmo->SetVisibility(TransformProxy ? TransformProxy->IsValid() : false);
	}
}

void USkeletalMeshGizmoWrapper::HandleBoneTransform(FGetTransform GetTransformFunc, FSetTransform SetTransformFunc)
{
	if (!TransformProxy || !Component.IsValid())
	{
		return;
	}

	static constexpr bool bModifyComponentOnTransform = false;
	TransformProxy->AddComponentCustom(Component.Get(),
		MoveTemp(GetTransformFunc),
		MoveTemp(SetTransformFunc),
		INDEX_NONE,
		bModifyComponentOnTransform);

	if (TransformGizmo)
	{
		TransformGizmo->SetVisibility(TransformProxy->IsValid());
	}
}