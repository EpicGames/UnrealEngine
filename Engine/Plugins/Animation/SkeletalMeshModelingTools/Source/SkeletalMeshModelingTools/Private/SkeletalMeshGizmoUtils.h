// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Containers/UnrealString.h"
#include "EditorGizmos/TransformGizmoInterfaces.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"

#include "SkeletalMeshGizmoUtils.generated.h"

#define UE_API SKELETALMESHMODELINGTOOLS_API

class UInteractiveToolsContext;
class UInteractiveToolManager;
class UTransformGizmo;
class USkeletonTransformProxy;
struct FGizmoCustomization;

namespace UE
{
	
namespace SkeletalMeshGizmoUtils
{
	/**
	 * The functions below are helper functions that simplify usage of a USkeletalMeshGizmoContextObject
	 * that is registered as a ContextStoreObject in an InteractiveToolsContext
	 */

	/**
	 * If one does not already exist, create a new instance of USkeletalMeshGizmoContextObject and add it to the
	 * ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore now has a USkeletalMeshGizmoContextObject (whether it already existed, or was created)
	 */
	SKELETALMESHMODELINGTOOLS_API bool RegisterTransformGizmoContextObject(UInteractiveToolsContext* InToolsContext);

	/**
	 * Remove any existing USkeletalMeshGizmoContextObject from the ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore no longer has a USkeletalMeshGizmoContextObject (whether it was removed, or did not exist)
	 */
	SKELETALMESHMODELINGTOOLS_API bool UnregisterTransformGizmoContextObject(UInteractiveToolsContext* InToolsContext);

	/**
	 * Spawn an editor like Transform Gizmo. ToolManager's ToolsContext must have a USkeletalMeshGizmoContextObject registered 
	 */
	SKELETALMESHMODELINGTOOLS_API UTransformGizmo* CreateTransformGizmo(UInteractiveToolManager* ToolManager, void* Owner = nullptr);
}
	
}

/**
 * USkeletalMeshGizmoWrapper is a wrapper class to handle a Transform Gizmo and it's Transform proxy so that it can
 * be used to update skeletal mesh infos using a Gizmo.
 */

UCLASS(MinimalAPI)
class USkeletalMeshGizmoWrapper : public USkeletalMeshGizmoWrapperBase
{
	GENERATED_BODY()
public:
	UE_API virtual void Initialize(const FTransform& InTransform = FTransform::Identity, const EToolContextCoordinateSystem& InTransformMode = EToolContextCoordinateSystem::Local) override;
	UE_API virtual void HandleBoneTransform(FGetTransform GetTransformFunc, FSetTransform SetTransformFunc) override;
	UE_API virtual void Clear() override;

	UE_API virtual bool CanInteract() const override;
	UE_API virtual bool IsGizmoHit(const FInputDeviceRay& PressPos) const override;

	UPROPERTY()
	TObjectPtr<UTransformGizmo> TransformGizmo;
	UPROPERTY()
	TObjectPtr<USkeletonTransformProxy> TransformProxy;
};

/**
 * USkeletalMeshGizmoContextObject is a utility object that registers a Gizmo Builder for UTransformGizmo.
 * (see UCombinedTransformGizmoContextObject for more details)
 */

UCLASS(MinimalAPI)
class USkeletalMeshGizmoContextObject : public USkeletalMeshGizmoContextObjectBase
{
	GENERATED_BODY()

public:

	// builder identifiers for transform gizmo
	static UE_API const FString& TransformBuilderIdentifier();
	
	UE_API void RegisterGizmosWithManager(UInteractiveToolManager* InToolManager);
	UE_API void UnregisterGizmosWithManager(UInteractiveToolManager* InToolManager);

	UE_API virtual USkeletalMeshGizmoWrapperBase* GetNewWrapper(UInteractiveToolManager* InToolManager, UObject* Outer = nullptr, IGizmoStateTarget* InStateTarget = nullptr, const TScriptInterface<ITransformGizmoSource>& GizmoSource = nullptr) override;

private:
	
	static UE_API const FGizmoCustomization& GetGizmoCustomization();
	
	bool bGizmosRegistered = false;
};

#undef UE_API
