// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshModelingToolsCommands.h"

#include "MeshGroupPaintTool.h"
#include "ModelingToolsActions.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshModelingToolsCommands"


void FSkeletalMeshModelingToolsCommands::RegisterCommands()
{
	UI_COMMAND(BeginSkeletalMeshRunMeshProcessorBlueprintTool, "Run Mesh Processor Blueprint", "Run a UEditorDynamicMeshProcessorBlueprint against the current mesh and preview the result in pose. Defaults to preview-only; toggle off Preview Only to commit the result back to the asset on Accept.", EUserInterfaceActionType::ToggleButton, FInputChord());

	// Match label / tooltip with FModelingToolsManagerCommands::LoadLodsTools so the Misc palette
	// reads the same as the corresponding palette in regular Modeling Mode.
	UI_COMMAND(LoadMiscTools, "Misc", "Additional Utility Tools", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ToggleEditingToolsMode, "Enable Editing Tools", "Toggles editing tools on or off.", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(NewMorphTarget, "New Morph Target", "Add a new Morph Target.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(AddMissingMorphTargetsFromSkeletalMesh, "Add Empty Morph Targets From Template...", "Pick a template Skeletal Mesh and add empty Morph Targets here for every name that exists on the template but not on the current mesh. Only Morph Target names are read; no delta data is transferred.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(MirrorSelectedMorphTarget, "Mirror Selected Morph Targets", "Mirror each selected Morph Target", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(MirrorEditingMorphTarget, "Mirror Editing Morph Target", "Mirror the Morph Target currently selected for editing, if geometry selection is active", EUserInterfaceActionType::Button, FInputChord(EKeys::M));

	UI_COMMAND(FlipSelectedMorphTarget, "Flip Selected Morph Targets", "Flip each selected Morph Target", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(FlipEditingMorphTarget, "Flip Editing Morph Target", "Flip the Morph Target currently selected for editing, if geometry selection is active", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(MergeSelectedMorphTargets, "Merge Selected Morph Targets", "Create a new Morph Target by summing the deltas of all selected Morph Targets", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ApplyCurrentWeightToMorphTarget, "Apply Current Weight To Morph Target", "Bake the current weight into the Morph Target deltas and reset the override weight to 1", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ToggleSelectedMorphTargetWeight, "Toggle Morph Target Weight (double click)", "Toggle the weight of the selected Morph Target(s) between 0 and 1. Double-clicking a row in the list does the same thing.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ConfigureMorphTargetNamingConvention, "Configure Naming Convention...", "Configure the wildcard patterns that identify left- and right-side Morph Targets for batch operations", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(GenerateFlippedMorphTargets, "Generate Flipped Morph Targets", "For each selected Morph Target whose name matches the configured left-side pattern, write a flipped copy of its deltas into the right-side counterpart (creating it if missing)", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(IsolateSelection, "Isolate Selection", "Show only the selected geometry, hiding everything else", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::H));
	UI_COMMAND(HideSelection, "Hide Selected", "Hide the selected geometry from the viewport", EUserInterfaceActionType::Button, FInputChord(EKeys::H));
	UI_COMMAND(ShowFullMesh, "Show Full Mesh", "Restore visibility of all geometry", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::H));

	UI_COMMAND(BeginSelectionAction_SelectAll, "Select All", "Select all elements", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A));
	UI_COMMAND(BeginSelectionAction_Invert, "Invert Selection", "Invert the current selection", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::I));
	UI_COMMAND(BeginSelectionAction_ExpandToConnected, "Expand To Connected (double click)", "Expand the selection to all geometrically-connected elements", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BeginSelectionAction_InvertConnected, "Invert Connected", "Invert the current selection across geometrically-connected elements", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control|EModifierKey::Shift, EKeys::I));
	UI_COMMAND(BeginSelectionAction_Expand, "Expand Selection", "Expand the current selection by one ring of elements", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Period));
	UI_COMMAND(BeginSelectionAction_Contract, "Contract Selection", "Contract the current selection by one ring of elements", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Comma));

	UI_COMMAND(ShowQuickAccessMenu, "Quick Access Menu", "Open a context menu at the cursor with selection-mode toggles and the most recently used tool", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::Q));

	UI_COMMAND(ToggleHotkeyHints, "Toggle Hotkey Hints", "Show/hide the active tool's hotkey hints overlay in the viewport", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::Slash));

	UI_COMMAND(ToggleShowAllLocalRotationAxes, "Show All Local Rotation Axes", "When enabled shows all local rotation axes, otherwise for the selected bone only", EUserInterfaceActionType::ToggleButton, FInputChord());	
}

const FSkeletalMeshModelingToolsCommands& FSkeletalMeshModelingToolsCommands::Get()
{
	return TCommands<FSkeletalMeshModelingToolsCommands>::Get();
}

FSkeletalMeshModelingToolsActionCommands::FSkeletalMeshModelingToolsActionCommands() : 
	TInteractiveToolCommands<FSkeletalMeshModelingToolsActionCommands>(
		"SeletalMeshModelingToolsEditMode", // Context name for fast lookup
		NSLOCTEXT("Contexts", "SeletalMeshModelingToolsEditMode", "Skeletal Mesh Modeling Tools - Shared Shortcuts"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}

void FSkeletalMeshModelingToolsActionCommands::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs)
{}

namespace UE::SkeletalMeshModelingTools
{

#define DEFINE_TOOL_ACTION_COMMANDS(CommandsClassName, ContextNameString, ContextLocKey, SettingsDialogString, ToolClassName) \
class CommandsClassName : public TInteractiveToolCommands<CommandsClassName> \
{ \
public: \
	CommandsClassName() \
		: TInteractiveToolCommands<CommandsClassName>(ContextNameString, NSLOCTEXT("Contexts", ContextLocKey, SettingsDialogString), NAME_None, FAppStyle::GetAppStyleSetName()) {} \
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override \
	{ \
		ToolCDOs.Add(GetMutableDefault<ToolClassName>()); \
	} \
};

DEFINE_TOOL_ACTION_COMMANDS(FSkeletonEditingToolActionCommands,
	USkeletonEditingTool::ActionCommandsContextName, "SkeletalMeshModelingToolsSkeletonEditing",
	"Skeletal Mesh Modeling Tools - Skeleton Editing Tool", USkeletonEditingTool);
DEFINE_TOOL_ACTION_COMMANDS(FSkinWeightsPaintToolActionCommands,
	USkinWeightsPaintTool::ActionCommandsContextName, "SkeletalMeshModelingToolsSkinWeightsPaintTool",
	"Skeletal Mesh Modeling Tools - Skin Weights Paint Tool", USkinWeightsPaintTool);
DEFINE_TOOL_ACTION_COMMANDS(FMeshGroupPaintToolActionCommands,
	TEXT("SkeletalMeshModelingToolsMeshGroupPaintTool"), "SkeletalMeshModelingToolsMeshGroupPaintTool",
	"Skeletal Mesh Modeling Tools - Group Paint Tool", UMeshGroupPaintTool);

}

void FSkeletalMeshModelingToolsActionCommands::RegisterAllToolActions()
{
	UE::SkeletalMeshModelingTools::FSkeletonEditingToolActionCommands::Register();
	UE::SkeletalMeshModelingTools::FSkinWeightsPaintToolActionCommands::Register();
	UE::SkeletalMeshModelingTools::FMeshGroupPaintToolActionCommands::Register();
}

void FSkeletalMeshModelingToolsActionCommands::UnregisterAllToolActions()
{
	UE::SkeletalMeshModelingTools::FSkeletonEditingToolActionCommands::Unregister();
	UE::SkeletalMeshModelingTools::FSkinWeightsPaintToolActionCommands::Unregister();
	UE::SkeletalMeshModelingTools::FMeshGroupPaintToolActionCommands::Unregister();
}

void FSkeletalMeshModelingToolsActionCommands::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
{
#define UPDATE_BINDING(CommandsType) \
	if (!bUnbind) { CommandsType::Get().BindCommandsForCurrentTool(UICommandList, Tool); } \
	else { CommandsType::Get().UnbindActiveCommands(UICommandList); }

	if (ExactCast<USkeletonEditingTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(UE::SkeletalMeshModelingTools::FSkeletonEditingToolActionCommands);
	}

	if (ExactCast<USkinWeightsPaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(UE::SkeletalMeshModelingTools::FSkinWeightsPaintToolActionCommands);
	}
	
	if (ExactCast<UMeshGroupPaintTool>(Tool) != nullptr)
	{
		UPDATE_BINDING(UE::SkeletalMeshModelingTools::FMeshGroupPaintToolActionCommands);
	}
}



#undef LOCTEXT_NAMESPACE
