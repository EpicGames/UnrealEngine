// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"
#include "Tools/InteractiveToolsCommands.h"
#include "SkeletalMeshModelingToolsStyle.h"


class FSkeletalMeshModelingToolsCommands : public TCommands<FSkeletalMeshModelingToolsCommands>
{
public:
	FSkeletalMeshModelingToolsCommands()
	    : TCommands<FSkeletalMeshModelingToolsCommands>(
	    	TEXT("SkeletalMeshModelingTools"),
	    	NSLOCTEXT("Contexts", "SkeletalMeshModelingToolsCommands", "Skeletal Mesh Modeling Tools"),
	    	NAME_None,
	    	FSkeletalMeshModelingToolsStyle::Get().GetStyleSetName())
	{
	}

	virtual void RegisterCommands() override;
	static const FSkeletalMeshModelingToolsCommands& Get();

	// Tool launch commands
	TSharedPtr<FUICommandInfo> BeginSkeletalMeshRunMeshProcessorBlueprintTool;

	// Toolkit palette command — labels the "Misc" tool group.
	TSharedPtr<FUICommandInfo> LoadMiscTools;

	// Editing tools commands
	TSharedPtr<FUICommandInfo> ToggleEditingToolsMode;

	// Morph Target Editing commands
	TSharedPtr<FUICommandInfo> NewMorphTarget;

	TSharedPtr<FUICommandInfo> AddMissingMorphTargetsFromSkeletalMesh;

	TSharedPtr<FUICommandInfo> MirrorSelectedMorphTarget;
	
	TSharedPtr<FUICommandInfo> MirrorEditingMorphTarget;

	TSharedPtr<FUICommandInfo> FlipSelectedMorphTarget;

	TSharedPtr<FUICommandInfo> FlipEditingMorphTarget;

	TSharedPtr<FUICommandInfo> MergeSelectedMorphTargets;

	TSharedPtr<FUICommandInfo> ApplyCurrentWeightToMorphTarget;

	TSharedPtr<FUICommandInfo> ToggleSelectedMorphTargetWeight;

	TSharedPtr<FUICommandInfo> ConfigureMorphTargetNamingConvention;

	TSharedPtr<FUICommandInfo> GenerateFlippedMorphTargets;

	// Geometry Isolation commands
	TSharedPtr<FUICommandInfo> IsolateSelection;
	TSharedPtr<FUICommandInfo> HideSelection;
	TSharedPtr<FUICommandInfo> ShowFullMesh;

	// Mirror the entries in FModelingToolsManagerCommands so chords are remappable
	// independently from the Modeling Mode shortcuts.
	TSharedPtr<FUICommandInfo> BeginSelectionAction_SelectAll;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Invert;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_ExpandToConnected;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_InvertConnected;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Expand;
	TSharedPtr<FUICommandInfo> BeginSelectionAction_Contract;

	// Opens a context-menu-style popup at the cursor with selection-mode toggles and the last-used tool.
	TSharedPtr<FUICommandInfo> ShowQuickAccessMenu;

	TSharedPtr<FUICommandInfo> ToggleHotkeyHints;

    /** Command to toggle show all local rotation axes */
	TSharedPtr<FUICommandInfo> ToggleShowAllLocalRotationAxes;
};

class FSkeletalMeshModelingToolsActionCommands : public TInteractiveToolCommands<FSkeletalMeshModelingToolsActionCommands>
{
public:
	FSkeletalMeshModelingToolsActionCommands();

	// TInteractiveToolCommands
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;
	
	static void RegisterAllToolActions();
	static void UnregisterAllToolActions();
	static void UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind = false);
};
