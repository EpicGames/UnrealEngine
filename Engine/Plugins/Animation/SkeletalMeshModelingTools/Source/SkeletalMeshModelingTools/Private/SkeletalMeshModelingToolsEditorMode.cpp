// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshModelingToolsEditorMode.h"

#include "AttributeEditorTool.h"
#include "ContextObjectStore.h"
#include "SkeletalMeshModelingToolsEditorModeToolkit.h"
#include "SkeletalMeshModelingToolsCommands.h"
#include "SkeletalMeshGizmoUtils.h"

#include "BaseGizmos/TransformGizmoUtil.h"
#include "EdMode.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "IPersonaEditorModeManager.h"
#include "MeshAttributePaintToolV2.h"
#include "ShowFlagMenuCommands.h"
#include "Features/IModularFeatures.h"

#include "SkeletalMeshModelingToolsModule.h"
#include "Selection/SKMGeometrySelectionTransformTweaker.h"
#include "ViewportInteractions/SKMModelingToolsGeometryClickSelection.h"
#include "ViewportInteractions/SKMModelingToolsGeometryDoubleClickSelection.h"
#include "ViewportInteractions/SKMModelingToolsGeometryFrustumSelection.h"
#include "ViewportInteractions/SKMModelingToolsGeometryHoverInteraction.h"
#include "ViewportInteractions/SKMModelingToolsGeometrySoftSelectRadiusDrag.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportMoveYawInteraction.h"
#include "ViewportInteractions/ViewportOrbitInteraction.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SSKMQuickAccessMenu.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Styling/StyleColors.h"
#include "Misc/MessageDialog.h"
#include "HAL/ConsoleManager.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshModelingToolsEditorMode)

// Stylus support is currently disabled due to issues with the stylus plugin
// We are leaving the code in this cpp file, defined out, so that it is easier to bring back if/when the stylus plugin is improved.

#define ENABLE_STYLUS_SUPPORT 0

#if ENABLE_STYLUS_SUPPORT 
#include "IStylusInputModule.h"
#include "IStylusState.h"
#endif

#include "AnimationEditorViewportClient.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include "ConvertToPolygonsTool.h"
#include "DeformMeshPolygonsTool.h"
#include "DisplaceMeshTool.h"
#include "DynamicMeshSculptTool.h"
#include "EditMeshPolygonsTool.h"
#include "HoleFillTool.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "ISkeletalMeshEditor.h"
#include "LatticeDeformerTool.h"
#include "MeshAttributePaintTool.h"
#include "MeshGroupPaintTool.h"
#include "MeshSpaceDeformerTool.h"
#include "MeshVertexPaintTool.h"
#include "MeshVertexSculptTool.h"
#include "ModelingToolsManagerActions.h"
#include "OffsetMeshTool.h"
#include "PersonaModule.h"
#include "PolygonOnMeshTool.h"
#include "ProjectToTargetTool.h"
#include "RemeshMeshTool.h"
#include "RemoveOccludedTrianglesTool.h"
#include "SimplifyMeshTool.h"
#include "SkeletalMeshEditingCache.h"
#include "SkeletalMeshEditorUtils.h"
#include "SmoothMeshTool.h"
#include "ToolTargetManager.h"
#include "WeldMeshEdgesTool.h"

#include "SkeletalMeshNotifier.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Components/SKMBackedDynaMeshComponent.h"
#include "SkeletalMesh/RefSkeletonPoser.h"


#include "SkeletalMesh/IHotkeyHintProvider.h"
#include "SkeletalMesh/ISkeletalMeshGeometryIsolationAwareTool.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "Tools/SKMRunMeshProcessorBlueprintTool.h"
#include "BakeMeshAttributeVertexTool.h"
#include "SkeletalMesh/SkinWeightsBindingTool.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"
#include "SkeletalMesh/SkeletalMeshVertexAttributePaintTool.h"
#include "ToolTargets/SkeletalMeshToolTarget.h"
#include "ToolTargets/SKMBackedDynaMeshComponentToolTarget.h"

#include "ModelingToolsEditorModeSettings.h"
#include "Selection/GeometrySelectionManager.h"
#include "Commands/ModifyGeometrySelectionCommand.h"
#include "Selection/SKMEditingCacheSelector.h"
#include "Widgets/Input/SSpinBox.h"
#include "SkeletalMeshModelingToolsViewportToolbarExtension.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshModelingToolsEditorMode"


#if ENABLE_STYLUS_SUPPORT
// FStylusStateTracker registers itself as a listener for stylus events and implements
// the IToolStylusStateProviderAPI interface, which allows MeshSurfacePointTool implementations
 // to query for the pen pressure.
//
// This is kind of a hack. Unfortunately the current Stylus module is a Plugin so it
// cannot be used in the base ToolsFramework, and we need this in the Mode as a workaround.
//
class FStylusStateTracker : public IStylusMessageHandler, public IToolStylusStateProviderAPI
{
public:
	const IStylusInputDevice* ActiveDevice = nullptr;
	int32 ActiveDeviceIndex = -1;

	bool bPenDown = false;
	float ActivePressure = 1.0;

	FStylusStateTracker()
	{
		UStylusInputSubsystem* StylusSubsystem = GEditor->GetEditorSubsystem<UStylusInputSubsystem>();
		StylusSubsystem->AddMessageHandler(*this);

		ActiveDevice = FindFirstPenDevice(StylusSubsystem, ActiveDeviceIndex);
		bPenDown = false;
	}

	virtual ~FStylusStateTracker()
	{
		if (GEditor)
		{
			if (UStylusInputSubsystem* StylusSubsystem = GEditor->GetEditorSubsystem<UStylusInputSubsystem>())
			{
				StylusSubsystem->RemoveMessageHandler(*this);
			}
		}
	}

	void OnStylusStateChanged(const FStylusState& NewState, int32 StylusIndex) override
	{
		if (ActiveDevice == nullptr)
		{
			UStylusInputSubsystem* StylusSubsystem = GEditor->GetEditorSubsystem<UStylusInputSubsystem>();
			ActiveDevice = FindFirstPenDevice(StylusSubsystem, ActiveDeviceIndex);
			bPenDown = false;
		}
		if (ActiveDevice != nullptr && ActiveDeviceIndex == StylusIndex)
		{
			bPenDown = NewState.IsStylusDown();
			ActivePressure = NewState.GetPressure();
		}
	}


	bool HaveActiveStylusState() const
	{
		return ActiveDevice != nullptr && bPenDown;
	}

	static const IStylusInputDevice* FindFirstPenDevice(const UStylusInputSubsystem* StylusSubsystem, int32& ActiveDeviceOut)
	{
		int32 NumDevices = StylusSubsystem->NumInputDevices();
		for (int32 k = 0; k < NumDevices; ++k)
		{
			const IStylusInputDevice* Device = StylusSubsystem->GetInputDevice(k);
			const TArray<EStylusInputType>& Inputs = Device->GetSupportedInputs();
			for (EStylusInputType Input : Inputs)
			{
				if (Input == EStylusInputType::Pressure)
				{
					ActiveDeviceOut = k;
					return Device;
				}
			}
		}
		return nullptr;
	}



	// IToolStylusStateProviderAPI implementation
	virtual float GetCurrentPressure() const override
	{
		return (ActiveDevice != nullptr && bPenDown) ? ActivePressure : 1.0f;
	}

};
#endif // ENABLE_STYLUS_SUPPORT

bool bAllowEditingBaseMeshViaGeometrySelection = false;
static FAutoConsoleVariableRef CVarAllowEditingBaseMeshViaGeometrySelection(
	TEXT("SkeletalMeshModelingTools.AllowEditingBaseMeshViaGeometrySelection"),
	bAllowEditingBaseMeshViaGeometrySelection,
	TEXT("If false, base mesh can no longer be edited via geometry selection"));

static void ShowEditorMessage(ELogVerbosity::Type InMessageType, const FText& InMessage)
{
	FNotificationInfo Notification(InMessage);
	Notification.bUseSuccessFailIcons = true;
	Notification.ExpireDuration = 5.0f;

	SNotificationItem::ECompletionState State = SNotificationItem::CS_Success;

	switch(InMessageType)
	{
	case ELogVerbosity::Warning:
		UE_LOGF(LogSkeletalMeshModelingTools, Warning, "%ls", *InMessage.ToString());
		break;
	case ELogVerbosity::Error:
		State = SNotificationItem::CS_Fail;
		UE_LOGF(LogSkeletalMeshModelingTools, Error, "%ls", *InMessage.ToString());
		break;
	default:
		break; // don't log anything unless a warning or error
	}
	
	FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(State);
}

// NOTE: This is a simple proxy at the moment. In the future we want to pull in more of the 
// modeling tools as we add support in the skelmesh storage.

const FEditorModeID USkeletalMeshModelingToolsEditorMode::Id("SkeletalMeshModelingToolsEditorMode");

FSkeletalMeshModelingToolsEditorModeNotifier::FSkeletalMeshModelingToolsEditorModeNotifier(USkeletalMeshModelingToolsEditorMode* InEditorMode)
	:EditorMode(InEditorMode)
{
}

void FSkeletalMeshModelingToolsEditorModeNotifier::HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	if (Notifying())
	{
		return;
	}

	switch (InNotifyType)
	{
	case ESkeletalMeshNotifyType::BonesSelected:
		EditorMode->SetSelectedBones(BoneNames);
		break;
	}
	
	Notify(BoneNames, InNotifyType);
}

FSkeletalMeshModelingToolsEditorModeBinding::FSkeletalMeshModelingToolsEditorModeBinding(USkeletalMeshModelingToolsEditorMode* InEditorMode)
	: EditorMode(InEditorMode)
{
}

TSharedPtr<ISkeletalMeshNotifier> FSkeletalMeshModelingToolsEditorModeBinding::GetNotifier()
{
	if (!Notifier)
	{
		Notifier = MakeShared<FSkeletalMeshModelingToolsEditorModeNotifier>(EditorMode.Get());
	}
	
	return Notifier;
}

ISkeletalMeshEditorBinding::NameFunction FSkeletalMeshModelingToolsEditorModeBinding::GetNameFunction()
{
	// unused;
	return {};
}

TArray<FName> FSkeletalMeshModelingToolsEditorModeBinding::GetSelectedBones() const
{
	return EditorMode->GetSelectedBones();
}

USkeletalMeshModelingToolsEditorMode::USkeletalMeshModelingToolsEditorMode() 
{
	Info = FEditorModeInfo(Id, LOCTEXT("SkeletalMeshEditingMode", "Skeletal Mesh Editing"), FSlateIcon(), false);
}


USkeletalMeshModelingToolsEditorMode::USkeletalMeshModelingToolsEditorMode(FVTableHelper& Helper) :
	UBaseLegacyWidgetEdMode(Helper)
{
	
}


USkeletalMeshModelingToolsEditorMode::~USkeletalMeshModelingToolsEditorMode()
{
	// Implemented in the CPP file so that the destructor for TUniquePtr<FStylusStateTracker> gets correctly compiled.
}


void USkeletalMeshModelingToolsEditorMode::Initialize()
{
	UBaseLegacyWidgetEdMode::Initialize();
	SetFlags(RF_Transactional);
}


void USkeletalMeshModelingToolsEditorMode::Enter()
{
	using namespace UE::SkeletalMeshModelingTools;

	UEdMode::Enter();

	UEditorInteractiveToolsContext* EditorInteractiveToolsContext = GetInteractiveToolsContext(EToolsContextScope::Editor);
	bDeactivateOnPIEStartStateToRestore = EditorInteractiveToolsContext->GetDeactivateToolsOnPIEStart();
	EditorInteractiveToolsContext->SetDeactivateToolsOnPIEStart(false);

	// Opt out of the global PreSaveWorldWithContext-driven tool deactivation: this fires for any
	// dirty level (e.g. autosave) 
	bDeactivateOnSaveWorldStateToRestore = EditorInteractiveToolsContext->GetDeactivateToolsOnSaveWorld();
	EditorInteractiveToolsContext->SetDeactivateToolsOnSaveWorld(false);

	UEditorInteractiveToolsContext* InteractiveToolsContext = GetInteractiveToolsContext();

	if (TObjectPtr<UToolTargetManager> ToolTargetManager = InteractiveToolsContext->TargetManager)
	{
		// Special tool target factory to support fast tool switch for supported tools
		USkeletalMeshBackedDynamicMeshComponentToolTargetFactory* SKMDynaMeshFactory = NewObject<USkeletalMeshBackedDynamicMeshComponentToolTargetFactory>(ToolTargetManager);
		SKMDynaMeshFactory->Init(this);
		
		ToolTargetManager->AddTargetFactory(SKMDynaMeshFactory);
		ToolTargetManager->AddTargetFactory( NewObject<USkeletalMeshReadOnlyToolTargetFactory>(ToolTargetManager) );	
	
	}

#if ENABLE_STYLUS_SUPPORT
	StylusStateTracker = MakeUnique<FStylusStateTracker>();
#endif
	// register gizmo helper
	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(InteractiveToolsContext);
	UE::SkeletalMeshGizmoUtils::RegisterTransformGizmoContextObject(InteractiveToolsContext);
	UE::SkeletalMeshEditorUtils::RegisterEditorContextObject(InteractiveToolsContext);

	const FModelingToolsManagerCommands& ToolManagerCommands = FModelingToolsManagerCommands::Get();

	RegisterTool(ToolManagerCommands.BeginPolyEditTool, TEXT("BeginPolyEditTool"), NewObject<UEditMeshPolygonsToolBuilder>());
	UEditMeshPolygonsToolBuilder* TriEditBuilder = NewObject<UEditMeshPolygonsToolBuilder>();
	TriEditBuilder->bTriangleMode = true;
	RegisterTool(ToolManagerCommands.BeginTriEditTool, TEXT("BeginTriEditTool"), TriEditBuilder);
	RegisterTool(ToolManagerCommands.BeginPolyDeformTool, TEXT("BeginPolyDeformTool"), NewObject<UDeformMeshPolygonsToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginHoleFillTool, TEXT("BeginHoleFillTool"), NewObject<UHoleFillToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginPolygonCutTool, TEXT("BeginPolyCutTool"), NewObject<UPolygonOnMeshToolBuilder>());

	RegisterTool(ToolManagerCommands.BeginSimplifyMeshTool, TEXT("BeginSimplifyMeshTool"), NewObject<USimplifyMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginRemeshMeshTool, TEXT("BeginRemeshMeshTool"), NewObject<URemeshMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginWeldEdgesTool, TEXT("BeginWeldEdgesTool"), NewObject<UWeldMeshEdgesToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginRemoveOccludedTrianglesTool, TEXT("BeginRemoveOccludedTrianglesTool"), NewObject<URemoveOccludedTrianglesToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginProjectToTargetTool, TEXT("BeginProjectToTargetTool"), NewObject<UProjectToTargetToolBuilder>());
	
	RegisterTool(ToolManagerCommands.BeginPolyGroupsTool, TEXT("BeginPolyGroupsTool"), NewObject<UConvertToPolygonsToolBuilder>());
	UMeshGroupPaintToolBuilder* MeshGroupPaintToolBuilder = NewObject<UMeshGroupPaintToolBuilder>();
#if ENABLE_STYLUS_SUPPORT 
	MeshGroupPaintToolBuilder->StylusAPI = StylusStateTracker.Get();
#endif
	RegisterTool(ToolManagerCommands.BeginMeshGroupPaintTool, TEXT("BeginMeshGroupPaintTool"), MeshGroupPaintToolBuilder);
	
	UMeshVertexSculptToolBuilder* MoveVerticesToolBuilder = NewObject<UMeshVertexSculptToolBuilder>();
#if ENABLE_STYLUS_SUPPORT
	MoveVerticesToolBuilder->StylusAPI = StylusStateTracker.Get();
#endif
	RegisterTool(ToolManagerCommands.BeginSculptMeshTool, TEXT("BeginSculptMeshTool"), MoveVerticesToolBuilder);

	UDynamicMeshSculptToolBuilder* DynaSculptToolBuilder = NewObject<UDynamicMeshSculptToolBuilder>();
	DynaSculptToolBuilder->bEnableRemeshing = true;
#if ENABLE_STYLUS_SUPPORT
	DynaSculptToolBuilder->StylusAPI = StylusStateTracker.Get();
#endif
	RegisterTool(ToolManagerCommands.BeginRemeshSculptMeshTool, TEXT("BeginRemeshSculptMeshTool"), DynaSculptToolBuilder);
	
	RegisterTool(ToolManagerCommands.BeginSmoothMeshTool, TEXT("BeginSmoothMeshTool"), NewObject<USmoothMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginOffsetMeshTool, TEXT("BeginOffsetMeshTool"), NewObject<UOffsetMeshToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginMeshSpaceDeformerTool, TEXT("BeginMeshSpaceDeformerTool"), NewObject<UMeshSpaceDeformerToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginLatticeDeformerTool, TEXT("BeginLatticeDeformerTool"), NewObject<ULatticeDeformerToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginDisplaceMeshTool, TEXT("BeginDisplaceMeshTool"), NewObject<UDisplaceMeshToolBuilder>());

	RegisterTool(ToolManagerCommands.BeginAttributeEditorTool, TEXT("BeginAttributeEditorTool"), NewObject<UAttributeEditorToolBuilder>());
	bool bUseExperimentalAttributePaintTool = true;
	if (IConsoleVariable* Variable = IConsoleManager::Get().FindConsoleVariable(TEXT("ModelingTools.UseExperimentalMeshAttributePaintTool")))
	{
		if (Variable->GetBool() == false)
		{
			bUseExperimentalAttributePaintTool = false;
		}
	}
	if (bUseExperimentalAttributePaintTool)
	{
		RegisterTool(ToolManagerCommands.BeginMeshAttributePaintTool, TEXT("BeginMeshAttributePaintTool"), NewObject<USkeletalMeshVertexAttributePaintToolBuilder>());
	}
	else
	{
		RegisterTool(ToolManagerCommands.BeginMeshAttributePaintTool, TEXT("BeginMeshAttributePaintTool"), NewObject<UMeshAttributePaintToolBuilder>());
	}
	RegisterTool(ToolManagerCommands.BeginMeshVertexPaintTool, TEXT("BeginMeshVertexPaintTool"), NewObject<UMeshVertexPaintToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSkinWeightsPaintTool, TEXT("BeginSkinWeightsPaintTool"), NewObject<USkinWeightsPaintToolBuilder>());
	RegisterTool(ToolManagerCommands.BeginSkinWeightsBindingTool, TEXT("BeginSkinWeightsBindingTool"), NewObject<USkinWeightsBindingToolBuilder>());

	// Skeleton Editing
	RegisterTool(ToolManagerCommands.BeginSkeletonEditingTool, TEXT("BeginSkeletonEditingTool"), NewObject<USkeletonEditingToolBuilder>());


	RegisterTool(FSkeletalMeshModelingToolsCommands::Get().BeginSkeletalMeshRunMeshProcessorBlueprintTool, TEXT("BeginSkeletalMeshRunMeshProcessorBlueprintTool"), NewObject<USkeletalMeshRunMeshProcessorBlueprintToolBuilder>());

	RegisterTool(ToolManagerCommands.BeginBakeMeshAttributeVertexTool, TEXT("BeginBakeMeshAttributeVertexTool"), NewObject<UBakeMeshAttributeVertexToolBuilder>());

	// register extensions
	RegisterExtensions();
	
	// highlights skin weights tool by default
	GetInteractiveToolsContext()->ToolManager->SelectActiveToolType(EToolSide::Left, TEXT("BeginSkinWeightsPaintTool"));

	// record switching behavior to restore on exit
	ToolSwitchModeToRestoreOnExit = GetInteractiveToolsContext()->ToolManager->GetToolSwitchMode();
	// tool switch always saves current change to the editing cache
	GetInteractiveToolsContext()->ToolManager->SetToolSwitchMode(EToolManagerToolSwitchMode::AcceptIfAble);

	GetInteractiveToolsContext()->ToolManager->OnToolPostBuild.AddUObject(this, &USkeletalMeshModelingToolsEditorMode::OnToolPostBuild);
	GetInteractiveToolsContext()->ToolManager->OnToolEndedWithStatus.AddUObject(this, &USkeletalMeshModelingToolsEditorMode::OnToolEndedWithStatus);

	InitializeGeometrySelectionSystems();
	
	ExtendSkeletalMeshEditorViewportToolbar(*this, Toolkit->GetToolkitCommands());

	const TWeakPtr<FModeToolkit> WeakToolkit = GetToolkit();
	if (WeakToolkit.IsValid())
	{
		ExtendSkeletalMeshEditorViewportToolbar(*this, WeakToolkit.Pin()->GetToolkitCommands());
	}

	if (UModeManagerInteractiveToolsContext* const ModeManagerInteractiveToolsContext = GetModeManager()->GetInteractiveToolsContext())
	{
		ModeManagerInteractiveToolsContext->OnBuildViewportInteractions().AddUObject(this, &USkeletalMeshModelingToolsEditorMode::OnBuildViewportInteractions);
		ModeManagerInteractiveToolsContext->PostBuildViewportInteractions().AddUObject(this, &USkeletalMeshModelingToolsEditorMode::PostBuildViewportInteractions);
	}


	// Pre Editing Cache Tick watchers, can write to Editing Cache
	GeometrySelectionStateWatcher.Initialize(
		[this]()
			{
				return IsGeometrySelectionActive();
			},
		[this](bool InActive)
			{
				if (GetCurrentEditingCache())
				{
					GetCurrentEditingCache()->ToggleForceEnableDynamicMesh(InActive);
				}
				DesiredSkeletonState.bManipulation = !InActive;
				ToggleGeometrySelectionViewportInteractions(InActive);
			},
		IsGeometrySelectionActive());
	
	SkeletonStateUpdater.Initialize(
		[this]()
			{
				return GetSkeletonStateFactors();
			},
		[this](FSkeletonStateFactors InFactors)
			{
				if (GetCurrentEditingCache())
				{
					const bool bManipulation = !InFactors.bIsGeometrySelectionActive && InFactors.DesiredSkeletonState.bManipulation;
					GetCurrentEditingCache()->ToggleBoneManipulation(bManipulation);

					ESkeletonDrawMode SkeletonDrawMode = ESkeletonDrawMode::Hidden; 
					if (InFactors.DesiredSkeletonState.bVisible)
					{
						SkeletonDrawMode = ESkeletonDrawMode::Default;
						
						if (InFactors.bIsGeometrySelectionActive)
						{
							SkeletonDrawMode = ESkeletonDrawMode::GreyedOut;
						}
					}
					GetCurrentEditingCache()->SetSkeletonDrawMode(SkeletonDrawMode);
					
				}
			},
		GetSkeletonStateFactors());
	
	// Post Editing Cache Tick Watchers, only read from Editing Cache
	SkeletonTreeHostStateUpdater.Initialize(
		[this]()
			{
				FSkeletonTreeHostStateFactors State;
				State.bEnableDynamicMeshSkeleton = GetCurrentEditingCache() ? GetCurrentEditingCache()->IsDynamicMeshSkeletonEnabled() : false;
				State.bIsUsingToolSkeletonWidget = bIsUsingToolSkeletonWidget;
				State.bActiveToolModifiesSkeleton = bActiveToolModifiesSkeleton;
				return State;
			},
		[this](const FSkeletonTreeHostStateFactors& State)
			{
				if (!TypedToolkit.IsValid() || !GetEditor().IsValid())
				{
					return;
				}

				if (State.bActiveToolModifiesSkeleton)
				{
					TypedToolkit.Pin()->DisableEditingCacheSkeletonCommands();
				}
				else
				{
					if (State.bEnableDynamicMeshSkeleton)
					{
						TypedToolkit.Pin()->EnableDynamicMeshSkeletonCommands();
					}
					else
					{
						TypedToolkit.Pin()->EnableSkeletalMeshSkeletonCommands();
					}	
				}
				
				if (State.bIsUsingToolSkeletonWidget)
				{
					TypedToolkit.Pin()->ShowToolSkeletonTree();
				}
				else
				{
					if (State.bEnableDynamicMeshSkeleton)
					{
						TypedToolkit.Pin()->ShowDynamicMeshSkeletonTree();
					}
					else
					{
						TypedToolkit.Pin()->ShowSkeletalMeshSkeletonTree();
					}
				}
			},
		FSkeletonTreeHostStateFactors());
	
}

UDebugSkelMeshComponent* USkeletalMeshModelingToolsEditorMode::GetSkelMeshComponent() const
{
	if (!Editor.IsValid())
	{
		return nullptr;
	}
	
	TSharedRef<class IPersonaToolkit> PersonaToolKit = Editor.Pin()->GetPersonaToolkit();
	return PersonaToolKit->GetPreviewMeshComponent();
}


TSharedPtr<FTabManager> USkeletalMeshModelingToolsEditorMode::GetAssociatedTabManager()
{
	if (Editor.IsValid())
	{
		return Editor.Pin()->GetAssociatedTabManager();
	}

	return nullptr;
}

void USkeletalMeshModelingToolsEditorMode::BindToolSkeletonTree(ISkeletalMeshEditingInterface* InEditingInterface)
{
	SetActiveToolModifiesSkeleton(InEditingInterface->CanModifySkeleton());
	if (InEditingInterface->CanModifySkeleton())
	{
		// If the tool owns/modifies a skeleton, it should directly interface with the tool skeleton tree, so unbind notifier for the editing cache skeleton
		EditingCacheNotifierBindScope.Reset();
		GetCurrentEditingCache()->GetNotifier()->HandleNotification({}, ESkeletalMeshNotifyType::BonesSelected);
	}

	SetIsUsingToolSkeletonWidget(TypedToolkit.Pin()->BindToolSkeletonTree(InEditingInterface));
}

void USkeletalMeshModelingToolsEditorMode::UnbindToolSkeletonTree()
{
	TypedToolkit.Pin()->UnbindToolSkeletonTree();
	if (!EditingCacheNotifierBindScope.IsValid())
	{
		EditingCacheNotifierBindScope.Reset(
			new FSkeletalMeshNotifierBindScope(
				GetModeBinding()->GetNotifier(),
				GetCurrentEditingCache()->GetNotifier()));

		GetCurrentEditingCache()->GetNotifier()->HandleNotification({}, ESkeletalMeshNotifyType::HierarchyChanged);
		GetCurrentEditingCache()->GetNotifier()->HandleNotification(GetSelectedBones(), ESkeletalMeshNotifyType::BonesSelected);
	}

	SetIsUsingToolSkeletonWidget(false);
	SetActiveToolModifiesSkeleton(false);
}


void USkeletalMeshModelingToolsEditorMode::ToggleBoneManipulation(bool bEnable)
{
	DesiredSkeletonState.bManipulation = bEnable;

	SkeletonStateUpdater.CheckAndUpdate();
}

USkeletonModifier* USkeletalMeshModelingToolsEditorMode::GetSkeletonReader()
{
	return SkeletonReader;
}

void USkeletalMeshModelingToolsEditorMode::SetSelectedBones(const TArray<FName>& InSelectedBones)
{
	SelectedBones = InSelectedBones;
}

TArray<FName> USkeletalMeshModelingToolsEditorMode::GetSelectedBones() const 
{
	return SelectedBones;
}

FName USkeletalMeshModelingToolsEditorMode::GetEditingMorphTarget() const
{
	if (!GetCurrentEditingCache())
	{
		return NAME_None;
	}
	
	if (!GetCurrentEditingCache()->GetMorphTargets().Contains(EditingMorphTarget))
	{
		return NAME_None;
	}

	return EditingMorphTarget;
}

USkeletalMeshModelingToolsEditorMode::FToolReactivateScope::FToolReactivateScope(USkeletalMeshModelingToolsEditorMode* InMode)
	: Mode(InMode)
{
	if (Mode->ToolReactivateScopeDepth == 0)
	{
		UInteractiveToolManager* ToolManager = Mode->GetInteractiveToolsContext()->ToolManager;
		if (ToolManager->HasActiveTool(EToolSide::Left))
		{
			EToolShutdownType ShutdownType = ToolManager->CanAcceptActiveTool(EToolSide::Left)
				? EToolShutdownType::Accept
				: EToolShutdownType::Completed;
			ToolManager->DeactivateTool(EToolSide::Left, ShutdownType);
			bNeedsReactivate = true;
		}
	}
	Mode->ToolReactivateScopeDepth++;
}

USkeletalMeshModelingToolsEditorMode::FToolReactivateScope::~FToolReactivateScope()
{
	Mode->ToolReactivateScopeDepth--;
	if (Mode->ToolReactivateScopeDepth == 0 && bNeedsReactivate)
	{
		UInteractiveToolManager* ToolManager = Mode->GetInteractiveToolsContext()->ToolManager;
		ToolManager->ActivateTool(EToolSide::Left);
	}
}

void USkeletalMeshModelingToolsEditorMode::SetEditingMorphTarget(FName InMorphTarget)
{
	FToolReactivateScope ReactivateScope(this);

	// Such that EditingMorphTarget is recorded for undo/redo 
	Modify();
	EditingMorphTarget = InMorphTarget;
}



TArray<FName> USkeletalMeshModelingToolsEditorMode::GetMorphTargets()
{
	if (!GetCurrentEditingCache())
	{
		return {};
	}
	return GetCurrentEditingCache()->GetMorphTargets();
}

float USkeletalMeshModelingToolsEditorMode::GetMorphTargetWeight(FName MorphTarget)
{
	if (!GetCurrentEditingCache())
	{
		return 0; 
	}
	
	return GetCurrentEditingCache()->GetMorphTargetWeight(MorphTarget);
}

FName USkeletalMeshModelingToolsEditorMode::AddMorphTarget(FName InName)
{
	FToolReactivateScope ReactivateScope(this);

	FName NewMorphTarget = GetCurrentEditingCache()->AddMorphTarget(InName);
	GetCurrentEditingCache()->OverrideMorphTargetWeight(NewMorphTarget, 1.0f);
	
	SetEditingMorphTarget(NewMorphTarget);
	
	return NewMorphTarget;
}

TArray<FName> USkeletalMeshModelingToolsEditorMode::AddMorphTargetsIfMissing(const TArray<FName>& Names)
{
	FToolReactivateScope ReactivateScope(this);
	return GetCurrentEditingCache()->AddMorphTargetsIfMissing(Names);
}

FName USkeletalMeshModelingToolsEditorMode::RenameMorphTarget(FName OldName, FName NewName)
{
	FToolReactivateScope ReactivateScope(this);

	bool bIsRenamingEditingMorphTarget = (GetEditingMorphTarget() == OldName);

	FName FinalNewName = GetCurrentEditingCache()->RenameMorphTarget(OldName, NewName);

	if (bIsRenamingEditingMorphTarget)
	{
		SetEditingMorphTarget(FinalNewName);
	}
	
	return FinalNewName;
}

void USkeletalMeshModelingToolsEditorMode::RemoveMorphTargets(const TArray<FName>& Names)
{
	FToolReactivateScope ReactivateScope(this);

	bool bIsRemovingEditingMorphTarget = Names.Contains(GetEditingMorphTarget());

	GetCurrentEditingCache()->RemoveMorphTargets(Names);

	if (bIsRemovingEditingMorphTarget)
	{
		SetEditingMorphTarget(NAME_None);
	}
}

TArray<FName> USkeletalMeshModelingToolsEditorMode::DuplicateMorphTargets(const TArray<FName>& Names)
{
	FToolReactivateScope ReactivateScope(this);
	return GetCurrentEditingCache()->DuplicateMorphTargets(Names);
}

void USkeletalMeshModelingToolsEditorMode::MirrorMorphTargets(const TArray<FName>& Names)
{
	FToolReactivateScope ReactivateScope(this);
	GetCurrentEditingCache()->MirrorMorphTargets(Names);
}

void USkeletalMeshModelingToolsEditorMode::FlipMorphTargets(const TArray<FName>& Names)
{
	FToolReactivateScope ReactivateScope(this);
	GetCurrentEditingCache()->FlipMorphTargets(Names);
}

FName USkeletalMeshModelingToolsEditorMode::MergeMorphTargets(const TArray<FName>& Names)
{
	FToolReactivateScope ReactivateScope(this);
	return GetCurrentEditingCache()->MergeMorphTargets(Names);
}

void USkeletalMeshModelingToolsEditorMode::ApplyCurrentWeightToMorphTarget(FName Name)
{
	FToolReactivateScope ReactivateScope(this);
	GetCurrentEditingCache()->ApplyCurrentWeightToMorphTarget(Name);
}

void USkeletalMeshModelingToolsEditorMode::GenerateFlippedMorphTargets(const TArray<TPair<FName, FName>>& InPairs)
{
	FToolReactivateScope ReactivateScope(this);
	GetCurrentEditingCache()->GenerateFlippedMorphTargets(InPairs);
}

void USkeletalMeshModelingToolsEditorMode::SetMorphTargetWeight(FName MorphTarget, float Weight)
{
	return GetCurrentEditingCache()->HandleSetMorphTargetWeight(MorphTarget, Weight);
}

bool USkeletalMeshModelingToolsEditorMode::GetMorphTargetAutoFill(FName MorphTarget)
{
	if (!GetCurrentEditingCache())
	{
		return false; 
	}
	
	return GetCurrentEditingCache()->GetMorphTargetAutoFill(MorphTarget);
}

void USkeletalMeshModelingToolsEditorMode::SetMorphTargetAutoFill(FName MorphTarget, bool bAutoFill, float PreviousOverrideWeight)
{
	return GetCurrentEditingCache()->HandleSetMorphTargetAutoFill(MorphTarget, bAutoFill, PreviousOverrideWeight);
}


void USkeletalMeshModelingToolsEditorMode::Exit()
{
	using namespace UE::SkeletalMeshModelingTools;

	if (UModeManagerInteractiveToolsContext* const ModeManagerInteractiveToolsContext = GetModeManager()->GetInteractiveToolsContext())
	{
		ModeManagerInteractiveToolsContext->OnBuildViewportInteractions().RemoveAll(this);
		ModeManagerInteractiveToolsContext->PostBuildViewportInteractions().RemoveAll(this);
	}
	
	RemoveSkeletalMeshEditorViewportToolbarExtensions();


	SelectionTransformTweaker->Shutdown();
	
	// Shutdown SelectionManager. Wait until after Tool shutdown in case some restore-selection
	// is involved (although since we are exiting Mode this currently would never matter)
	if (SelectionManager != nullptr)
	{
		GetInteractiveToolsContext()->ContextObjectStore->RemoveContextObject(SelectionManager);
		SelectionManager->ClearSelection();
		SelectionManager->Shutdown();		// will clear active targets
		SelectionManager = nullptr;
	}

	
	if (GetCurrentEditingCache())
	{
		GetCurrentEditingCache()->Destroy();
		CurrentEditingCache = nullptr;
	}
	
	UEditorInteractiveToolsContext* InteractiveToolsContext = GetInteractiveToolsContext();
	UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(InteractiveToolsContext);
	UE::SkeletalMeshGizmoUtils::UnregisterTransformGizmoContextObject(InteractiveToolsContext);
	UE::SkeletalMeshEditorUtils::UnregisterEditorContextObject(InteractiveToolsContext);
	
	UEditorInteractiveToolsContext* EditorInteractiveToolsContext = GetInteractiveToolsContext(EToolsContextScope::Editor);
	EditorInteractiveToolsContext->SetDeactivateToolsOnPIEStart(bDeactivateOnPIEStartStateToRestore);
	EditorInteractiveToolsContext->SetDeactivateToolsOnSaveWorld(bDeactivateOnSaveWorldStateToRestore);

	// restore previous tool switching behavior
	GetInteractiveToolsContext()->ToolManager->SetToolSwitchMode(ToolSwitchModeToRestoreOnExit);

	if (Editor.IsValid())
	{
		Editor.Pin()->OnPreSaveAsset().RemoveAll(this);
		Editor.Pin()->OnPreSaveAssetAs().RemoveAll(this);
	}

	if (SkeletalMesh.IsValid())
	{
		SkeletalMesh->GetOnPreMeshChange().RemoveAll(this);
		SkeletalMesh->GetOnMeshChanged().RemoveAll(this);
	}
	
#if ENABLE_STYLUS_SUPPORT
	StylusStateTracker = nullptr;
#endif

	UEdMode::Exit();
}

void USkeletalMeshModelingToolsEditorMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	Super::Render(View, Viewport, PDI);

	if (GetCurrentEditingCache())
	{
		auto SetBoneDrawConfig = [Viewport](FSkelDebugDrawConfig& Config)
			{
				if (const FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(Viewport->GetClient()))
				{
					Config.BoneDrawMode = AnimViewportClient->GetBoneDrawMode();
					Config.BoneDrawSize = AnimViewportClient->GetBoneDrawSize();
				}
			};
		GetCurrentEditingCache()->Render(PDI, SetBoneDrawConfig);
	}
}

void USkeletalMeshModelingToolsEditorMode::RegisterExtensions()
{
	TArray<ISkeletalMeshModelingModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<ISkeletalMeshModelingModeToolExtension>(
		ISkeletalMeshModelingModeToolExtension::GetModularFeatureName());
	if (Extensions.IsEmpty())
	{
		return;
	}

	UEditorInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext();
	
	FExtensionToolQueryInfo ExtensionQueryInfo;
	ExtensionQueryInfo.ToolsContext = ToolsContext;
	ExtensionQueryInfo.AssetAPI = nullptr;
#if ENABLE_STYLUS_SUPPORT
	ExtensionQueryInfo.StylusAPI = StylusStateTracker.Get();
#endif

	for (ISkeletalMeshModelingModeToolExtension* Extension: Extensions)
	{
		TArray<FExtensionToolDescription> ToolSet;
		Extension->GetExtensionTools(ExtensionQueryInfo, ToolSet);
		for (const FExtensionToolDescription& ToolInfo : ToolSet)
		{
			RegisterTool(ToolInfo.ToolCommand, ToolInfo.ToolName.ToString(), ToolInfo.ToolBuilder);
			ExtensionToolToInfo.Add(ToolInfo.ToolName.ToString(), ToolInfo);
		}

		TArray<TSubclassOf<UToolTargetFactory>> ExtensionToolTargetFactoryClasses;
		if (Extension->GetExtensionToolTargets(ExtensionToolTargetFactoryClasses))
		{
			for (const TSubclassOf<UToolTargetFactory>& ExtensionTargetFactoryClass : ExtensionToolTargetFactoryClasses)
			{
				ToolsContext->TargetManager->AddTargetFactory(NewObject<UToolTargetFactory>(GetToolManager(), ExtensionTargetFactoryClass.Get()));
			}
		}
	}
}

bool USkeletalMeshModelingToolsEditorMode::TryGetExtensionToolCommandGetter(
	UInteractiveToolManager* InManager, const UInteractiveTool* InTool, 
	TFunction<const UE::IInteractiveToolCommandsInterface& ()>& GetterOut) const
{
	if (!ensure(InManager && InTool)
		|| InManager->GetActiveTool(EToolSide::Mouse) != InTool)
	{
		return false;
	}

	const FString ToolName = InManager->GetActiveToolName(EToolSide::Mouse);
	if (ToolName.IsEmpty())
	{
		return false;
	}

	const FExtensionToolDescription* ToolDescription = ExtensionToolToInfo.Find(ToolName);
	if (!ToolDescription || !ToolDescription->ToolCommandsGetter)
	{
		return false;
	}
	
	GetterOut = ToolDescription->ToolCommandsGetter;
	return true;
}

UGeometrySelectionManager* USkeletalMeshModelingToolsEditorMode::GetSelectionManager() const
{
	return SelectionManager;
}

void USkeletalMeshModelingToolsEditorMode::UpdateGeometrySelectionDerivedState()
{
	GeometrySelectionStateWatcher.CheckAndUpdate();
	SkeletonStateUpdater.CheckAndUpdate();
}

void USkeletalMeshModelingToolsEditorMode::SetSuspendGeometrySelection(bool bSuspend)
{
	if (bSuspendGeometrySelection == bSuspend)
	{
		return;
	}
	bSuspendGeometrySelection = bSuspend;

	UpdateGeometrySelectionDerivedState();
}

void USkeletalMeshModelingToolsEditorMode::SetIsUsingToolSkeletonWidget(bool bUsing)
{
	if (bIsUsingToolSkeletonWidget == bUsing)
	{
		return;
	}
	bIsUsingToolSkeletonWidget = bUsing;

	SkeletonTreeHostStateUpdater.CheckAndUpdate();
}

void USkeletalMeshModelingToolsEditorMode::SetActiveToolModifiesSkeleton(bool bModifies)
{
	if (bActiveToolModifiesSkeleton == bModifies)
	{
		return;
	}
	bActiveToolModifiesSkeleton = bModifies;

	SkeletonTreeHostStateUpdater.CheckAndUpdate();
}

void USkeletalMeshModelingToolsEditorMode::HandleEditingCacheComponentChanged()
{
	OnMorphTargetDataChangedDelegate.Broadcast();
}

void USkeletalMeshModelingToolsEditorMode::HandleEditingCacheSkeletonChanged()
{
	SkeletonReader->ExternalUpdate(GetCurrentEditingCache()->GetEditingMeshComponent()->GetRefSkeleton());
	TypedToolkit.Pin()->RefreshDynamicMeshSkeletonTreeIfNeeded();

	SkeletonTreeHostStateUpdater.CheckAndUpdate();
}

void USkeletalMeshModelingToolsEditorMode::HandleEditingCachePreviewMeshDeformed()
{
	SelectionTransformTweaker->UpdateSelectionFrameTransform();
}

void USkeletalMeshModelingToolsEditorMode::InitializeGeometrySelectionSystems()
{
	TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> WeakThis(this);
	
	const FModelingToolsManagerCommands& ToolManagerCommands = FModelingToolsManagerCommands::Get();
	
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
	
	// set up SelectionManager and register known factory types
	SelectionManager = NewObject<UGeometrySelectionManager>(GetToolManager());
	SelectionManager->Initialize(GetInteractiveToolsContext(), GetToolManager()->GetContextTransactionsAPI());
	{
		TUniquePtr<FSkeletalMeshEditingCacheSelectorFactory> SelectorFactory = MakeUnique<FSkeletalMeshEditingCacheSelectorFactory>();
		SelectorFactory->Init(this);
		SelectionManager->RegisterSelectorFactory(MoveTemp(SelectorFactory));
	}

	// Any selection-manager-driven state change (mode, selection content, undo/redo replay of
	// ChangeSelectionMode transactions) re-settles the derived state. Combined with the setter
	// for bSuspendGeometrySelection, every input to IsGeometrySelectionActive() has a causal fire.
	SelectionManager->OnSelectionModified.AddUObject(
		this, &USkeletalMeshModelingToolsEditorMode::UpdateGeometrySelectionDerivedState);
	// Matches ModelingToolsEditorMode, might be giving too much access... probably better wrap it in a interface
	GetInteractiveToolsContext()->ContextObjectStore->AddContextObject(SelectionManager);

	SelectionTransformTweaker= NewObject<USkeletalMeshGeometrySelectionTransformTweaker>();
	SelectionTransformTweaker->Setup(SelectionManager);
	
	const UModelingToolsModeCustomizationSettings* ModelingEditorSettings = GetDefault<UModelingToolsModeCustomizationSettings>();
	
	// Colors initialized here any time Modeling mode is entered
	SelectionManager->SetSelectionColors(ModelingEditorSettings->UnselectedColor, ModelingEditorSettings->HoverOverSelectedColor, ModelingEditorSettings->HoverOverUnselectedColor, ModelingEditorSettings->GeometrySelectedColor, ModelingEditorSettings->GeometrySoftSelectedColor);

	ActiveGeometrySelectionCommandList = MakeShared<FUICommandList>();

	auto RegisterSelectionCommand = [this, &CommandList](UGeometrySelectionEditCommand* Command, TSharedPtr<FUICommandInfo> UICommand, bool bAlwaysVisible)
	{
		ModelingModeCommands.Add(Command);
		const FUIAction Action(
			FExecuteAction::CreateUObject(GetSelectionManager(), &UGeometrySelectionManager::ExecuteSelectionCommand, Command),
			FCanExecuteAction::CreateUObject(GetSelectionManager(), &UGeometrySelectionManager::CanExecuteSelectionCommand, Command),
			FIsActionChecked(),
			(bAlwaysVisible) ? FIsActionButtonVisible() :
				FIsActionButtonVisible::CreateUObject(GetSelectionManager(), &UGeometrySelectionManager::CanExecuteSelectionCommand, Command),
			EUIActionRepeatMode::RepeatDisabled);
		CommandList->MapAction(UICommand, Action);
		ActiveGeometrySelectionCommandList->MapAction(UICommand, Action);
	};

	const FSkeletalMeshModelingToolsCommands& ModeCommands = FSkeletalMeshModelingToolsCommands::Get();
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand>(), ModeCommands.BeginSelectionAction_SelectAll, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_Invert>(), ModeCommands.BeginSelectionAction_Invert, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_ExpandToConnected>(), ModeCommands.BeginSelectionAction_ExpandToConnected, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_InvertConnected>(), ModeCommands.BeginSelectionAction_InvertConnected, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_Expand>(), ModeCommands.BeginSelectionAction_Expand, true);
	RegisterSelectionCommand(NewObject<UModifyGeometrySelectionCommand_Contract>(), ModeCommands.BeginSelectionAction_Contract, true);




	GetInteractiveToolsContext()->OnRender.AddUObject(this, &USkeletalMeshModelingToolsEditorMode::OnToolContextRender);
	GetInteractiveToolsContext()->OnDrawHUD.AddUObject(this, &USkeletalMeshModelingToolsEditorMode::OnToolContextDrawHUD);


	// Register Frame Modes
	auto ToggleFromGeometryAction = [WeakThis](EModelingSelectionInteraction_LocalFrameMode LocalFrameMode)
		{
			return FUIAction(
				FExecuteAction::CreateLambda([WeakThis, LocalFrameMode]
				{
					if (!WeakThis.IsValid())
					{
						return;
					}
					WeakThis->SelectionTransformTweaker->SetLocalFrameMode(LocalFrameMode);
					UModelingToolsModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
					ModelingEditorSettings->LastMeshSelectionLocalFrameMode = static_cast<int>(LocalFrameMode);
					ModelingEditorSettings->SaveConfig();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([WeakThis, LocalFrameMode]()
				{
					if (!WeakThis.IsValid())
					{
						return false;
					}
					return WeakThis->SelectionTransformTweaker->GetLocalFrameMode() == LocalFrameMode;
				}));
		};
	
	CommandList->MapAction(ToolManagerCommands.SelectionLocalFrameMode_Geometry,
		ToggleFromGeometryAction(EModelingSelectionInteraction_LocalFrameMode::FromGeometry));
	CommandList->MapAction(ToolManagerCommands.SelectionLocalFrameMode_Object,
		ToggleFromGeometryAction( EModelingSelectionInteraction_LocalFrameMode::FromObject));
	
	// Register Selection Hit Back Faces Toggle
	FUIAction ToggleHitBackFacesAction = FUIAction(
		FExecuteAction::CreateLambda([WeakThis]
		{
			UModelingToolsModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
			ModelingEditorSettings->bMeshSelectionHitBackFaces = !ModelingEditorSettings->bMeshSelectionHitBackFaces;
			WeakThis->SelectionManager->SetHitBackFaces(ModelingEditorSettings->bMeshSelectionHitBackFaces);
			ModelingEditorSettings->SaveConfig();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([WeakThis]()
		{
			if (!WeakThis.IsValid())
			{
				return false;
			}
			return WeakThis->SelectionManager->GetHitBackFaces();
		}));
	CommandList->MapAction(ToolManagerCommands.SelectionHitBackFaces, ToggleHitBackFacesAction);

	// Register Geometry Isolation commands
	CommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().IsolateSelection, FUIAction(
		FExecuteAction::CreateLambda([WeakThis]
		{
			if (!WeakThis.IsValid() || !WeakThis->GetToolManager() || !WeakThis->GetToolManager()->GetContextTransactionsAPI())
			{
				return;
			}
			IToolsContextTransactionsAPI* TransactionsAPI = WeakThis->GetToolManager()->GetContextTransactionsAPI();
			TransactionsAPI->BeginUndoTransaction(LOCTEXT("IsolateSelectionTransaction", "Isolate Selection"));
			if (!WeakThis->IsolateSelection())
			{
				TransactionsAPI->CancelUndoTransaction();
				return;
			}

			TransactionsAPI->EndUndoTransaction();
		}),
		FCanExecuteAction::CreateLambda([WeakThis]()
		{
			return WeakThis.IsValid() && WeakThis->HasActiveGeometrySelection();
		})));

	CommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().HideSelection, FUIAction(
		FExecuteAction::CreateLambda([WeakThis]
		{
			if (!WeakThis.IsValid() || !WeakThis->GetToolManager() || !WeakThis->GetToolManager()->GetContextTransactionsAPI())
			{
				return;
			}
			IToolsContextTransactionsAPI* TransactionsAPI = WeakThis->GetToolManager()->GetContextTransactionsAPI();
			TransactionsAPI->BeginUndoTransaction(LOCTEXT("HideSelectionTransaction", "Hide Selection"));
			if (!WeakThis->HideSelection())
			{
				TransactionsAPI->CancelUndoTransaction();
				return;
			}

			TransactionsAPI->EndUndoTransaction();
		}),
		FCanExecuteAction::CreateLambda([WeakThis]()
		{
			return WeakThis.IsValid() && WeakThis->HasActiveGeometrySelection();
		})));

	CommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().ShowFullMesh, FUIAction(
		FExecuteAction::CreateLambda([WeakThis]
		{
			if (!WeakThis.IsValid() || !WeakThis->GetToolManager() || !WeakThis->GetToolManager()->GetContextTransactionsAPI())
			{
				return;
			}
			IToolsContextTransactionsAPI* TransactionsAPI = WeakThis->GetToolManager()->GetContextTransactionsAPI();
			TransactionsAPI->BeginUndoTransaction(LOCTEXT("ShowFullMeshTransaction", "Show Full Mesh"));
			if (!WeakThis->ShowFullMesh())
			{
				TransactionsAPI->CancelUndoTransaction();
				return;
			}
				
			TransactionsAPI->EndUndoTransaction();
		}),
		FCanExecuteAction::CreateLambda([WeakThis]()
		{
			return WeakThis.IsValid() && WeakThis->CurrentEditingCache && WeakThis->CurrentEditingCache->HasIsolation();
		})));

	// Register Soft Selection Toggle
	FUIAction ToggleSoftSelectionAction = FUIAction(
		FExecuteAction::CreateLambda([WeakThis]
		{
			if (!WeakThis.IsValid())
			{
				return;
			}
			bool bIsSoftSelectionEnabled = WeakThis->SelectionManager->IsSoftSelectionEnabled();
			WeakThis->SelectionManager->SetSoftSelectionEnabled(!bIsSoftSelectionEnabled);
		}),
		FCanExecuteAction::CreateLambda([WeakThis]()
		{
			if (!WeakThis.IsValid())
			{
				return false;
			}
			return WeakThis->IsGeometrySelectionActive();	
		}),
		FIsActionChecked::CreateLambda([WeakThis]()
		{
			if (!WeakThis.IsValid())
			{
				return false;
			}
			return WeakThis->SelectionManager->IsSoftSelectionEnabled();
		}));
	CommandList->MapAction(ToolManagerCommands.SelectionEnableSoftSelection, ToggleSoftSelectionAction);

	// Register Morph Target operations
	FUIAction MirrorEditingMorphTargetAction = FUIAction(
		FExecuteAction::CreateLambda([WeakThis]
		{
			if (!WeakThis.IsValid() || !WeakThis->GetToolManager() || !WeakThis->GetToolManager()->GetContextTransactionsAPI())
			{
				return;
			}
				
			FName MorphTargetName = WeakThis->GetEditingMorphTarget();
			FText Title = FText::Format( LOCTEXT("MirrorEditingMorphTransaction", "Mirror {0}"), FText::FromName(MorphTargetName));
			IToolsContextTransactionsAPI* TransactionsAPI = WeakThis->GetToolManager()->GetContextTransactionsAPI();
			TransactionsAPI->BeginUndoTransaction(Title);
				
			WeakThis->MirrorMorphTargets({MorphTargetName});
				
			TransactionsAPI->EndUndoTransaction();
		}),
		FCanExecuteAction::CreateLambda([WeakThis]()
		{
			if (!WeakThis.IsValid())
			{
				return false;
			}
			return WeakThis->GetEditingMorphTarget() != NAME_None && WeakThis->IsGeometrySelectionActive();
		}));
	CommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().MirrorEditingMorphTarget, MirrorEditingMorphTargetAction);

	// Register Flip Morph Target operation
	FUIAction FlipEditingMorphTargetAction = FUIAction(
		FExecuteAction::CreateLambda([WeakThis]
		{
			if (!WeakThis.IsValid() || !WeakThis->GetToolManager() || !WeakThis->GetToolManager()->GetContextTransactionsAPI())
			{
				return;
			}

			FName MorphTargetName = WeakThis->GetEditingMorphTarget();
			FText Title = FText::Format( LOCTEXT("FlipEditingMorphTransaction", "Flip {0}"), FText::FromName(MorphTargetName));
			IToolsContextTransactionsAPI* TransactionsAPI = WeakThis->GetToolManager()->GetContextTransactionsAPI();
			TransactionsAPI->BeginUndoTransaction(Title);

			WeakThis->FlipMorphTargets({MorphTargetName});

			TransactionsAPI->EndUndoTransaction();
		}),
		FCanExecuteAction::CreateLambda([WeakThis]()
		{
			if (!WeakThis.IsValid())
			{
				return false;
			}
			return WeakThis->GetEditingMorphTarget() != NAME_None && WeakThis->IsGeometrySelectionActive();
		}));
	CommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().FlipEditingMorphTarget, FlipEditingMorphTargetAction);

	CommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().ShowQuickAccessMenu,
		FExecuteAction::CreateUObject(this, &USkeletalMeshModelingToolsEditorMode::ShowQuickAccessMenu));

	CommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().ToggleHotkeyHints,
		FExecuteAction::CreateLambda([WeakThis]()
		{
			if (WeakThis.IsValid())
			{
				WeakThis->bShowHotkeyHints = !WeakThis->bShowHotkeyHints;
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([WeakThis]()
		{
			return WeakThis.IsValid() && WeakThis->bShowHotkeyHints;
		}));

	// set up selection type toggles
	auto RegisterSelectionMode = [WeakThis](UGeometrySelectionManager::EMeshTopologyMode TopoMode, UE::Geometry::EGeometryElementType ElementMode, TSharedPtr<FUICommandInfo> UICommand)
	{
		WeakThis->Toolkit->GetToolkitCommands()->MapAction(UICommand,
			FExecuteAction::CreateLambda([WeakThis, TopoMode, ElementMode]() {
				if ( WeakThis.IsValid() && WeakThis->GetToolManager() && WeakThis->GetToolManager()->GetContextTransactionsAPI())
				{
					// Accept/complete active tool before switching selection mode
					UInteractiveToolManager* ToolManager = WeakThis->GetToolManager();
					if (ToolManager->HasActiveTool(EToolSide::Left))
					{
						EToolShutdownType ShutdownType = ToolManager->CanAcceptActiveTool(EToolSide::Left)
							? EToolShutdownType::Accept
							: EToolShutdownType::Completed;
						ToolManager->DeactivateTool(EToolSide::Left, ShutdownType);
					}

					IToolsContextTransactionsAPI* TransactionsAPI = ToolManager->GetContextTransactionsAPI();
					TransactionsAPI->BeginUndoTransaction(LOCTEXT("ChangeSelectionMode", "Selection Mode"));
					if (WeakThis->GetCurrentEditingCache() && WeakThis->GetCurrentEditingCache()->HasIsolation())
					{
						UE::Geometry::EGeometryTopologyType NewTopologyType = (TopoMode == UGeometrySelectionManager::EMeshTopologyMode::Polygroup)
							? UE::Geometry::EGeometryTopologyType::Polygroup
							: UE::Geometry::EGeometryTopologyType::Triangle;
						WeakThis->GetCurrentEditingCache()->ConvertIsolationForTopologyMode(NewTopologyType);
					}
					WeakThis->SelectionManager->SetMeshSelectionTypeAndMode(ElementMode, TopoMode,  TopoMode != UGeometrySelectionManager::EMeshTopologyMode::None);
					TransactionsAPI->EndUndoTransaction();
					
					if (UModelingToolsModeCustomizationSettings* ModelingEditorSettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>())
					{
						ModelingEditorSettings->LastMeshSelectionTopologyMode = static_cast<int>(TopoMode);
						ModelingEditorSettings->LastMeshSelectionElementType = static_cast<int>(ElementMode);
						ModelingEditorSettings->SaveConfig();
					}
				}
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([WeakThis, TopoMode, ElementMode]()
				{
					if (!WeakThis.IsValid() || !WeakThis->SelectionManager)
					{
						return false;
					}
					
					// Don't want to show the current active mode when a tool is running, it would be quite confusing
					if (WeakThis->bSuspendGeometrySelection)
					{
						return false;
					}

					return WeakThis->SelectionManager->GetMeshTopologyMode() == TopoMode && WeakThis->SelectionManager->GetSelectionElementType() == ElementMode;
				}),
			EUIActionRepeatMode::RepeatDisabled);
	};
	{
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::None, UGeometrySelectionManager::EGeometryElementType::Face, ToolManagerCommands.MeshSelectionModeAction_NoSelection);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Triangle, UGeometrySelectionManager::EGeometryElementType::Face, ToolManagerCommands.MeshSelectionModeAction_MeshTriangles);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Triangle, UGeometrySelectionManager::EGeometryElementType::Vertex, ToolManagerCommands.MeshSelectionModeAction_MeshVertices);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Triangle, UGeometrySelectionManager::EGeometryElementType::Edge, ToolManagerCommands.MeshSelectionModeAction_MeshEdges);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Polygroup, UGeometrySelectionManager::EGeometryElementType::Face, ToolManagerCommands.MeshSelectionModeAction_GroupFaces);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Polygroup, UGeometrySelectionManager::EGeometryElementType::Vertex, ToolManagerCommands.MeshSelectionModeAction_GroupCorners);
		RegisterSelectionMode(UGeometrySelectionManager::EMeshTopologyMode::Polygroup, UGeometrySelectionManager::EGeometryElementType::Edge, ToolManagerCommands.MeshSelectionModeAction_GroupEdges);
	}
}

void USkeletalMeshModelingToolsEditorMode::OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source)
{
	USkeletalMeshModelingToolsGeometryFrustumSelection* GeometryFrustumSelection =
		Source->AddInteraction<USkeletalMeshModelingToolsGeometryFrustumSelection>();
	GeometryFrustumSelection->BindSelectionManager(SelectionManager);

	USkeletalMeshModelingToolsGeometryClickSelection* GeometryClickSelection =
		Source->AddInteraction<USkeletalMeshModelingToolsGeometryClickSelection>();
	GeometryClickSelection->BindSelectionManager(SelectionManager);

	USkeletalMeshModelingToolsGeometryDoubleClickSelection* GeometryDoubleClickSelection =
		Source->AddInteraction<USkeletalMeshModelingToolsGeometryDoubleClickSelection>();
	{
		TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> WeakThis(this);
		GeometryDoubleClickSelection->BindSelection(SelectionManager,
			[WeakThis]() -> UPrimitiveComponent*
			{
				if (USkeletalMeshModelingToolsEditorMode* Mode = WeakThis.Get())
				{
					if (USkeletalMeshEditingCache* Cache = Mode->GetCurrentEditingCache())
					{
						return Cache->GetEditingMeshComponent();
					}
				}
				return nullptr;
			});
	}

	USkeletalMeshModelingToolsGeometryHoverInteraction* GeometryHoverInteraction =
		Source->AddInteraction<USkeletalMeshModelingToolsGeometryHoverInteraction>();
	GeometryHoverInteraction->BindSelectionManager(SelectionManager);

	USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction* GeometrySoftSelectDrag =
		Source->AddInteraction<USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction>();
	GeometrySoftSelectDrag->BindSelectionManager(SelectionManager);
}

void USkeletalMeshModelingToolsEditorMode::PostBuildViewportInteractions(const UViewportInteractionsBehaviorSource* Source)
{
	// Since viewport interactions are rebuilt whenever an editor mode is changed (including modes that can coexists like the persona skeleton selection edit mode)
	// we have to check again to make sure the correct interactions are enabled when that happens
	ToggleGeometrySelectionViewportInteractions(IsGeometrySelectionActive());
}

bool USkeletalMeshModelingToolsEditorMode::IsGeometrySelectionActive() const
{
	return !bSuspendGeometrySelection && SelectionManager->GetMeshTopologyMode() != UGeometrySelectionManager::EMeshTopologyMode::None;
}

bool USkeletalMeshModelingToolsEditorMode::HasActiveGeometrySelection() const
{
	return IsGeometrySelectionActive() && SelectionManager->HasSelection();
}

void USkeletalMeshModelingToolsEditorMode::ToggleGeometrySelectionViewportInteractions(bool bEnable)
{
	if (GetModeManager()->GetInteractiveToolsContext()->UsesViewportInteractions())
	{
		UViewportInteractionsBehaviorSource* BehaviorSource = GetModeManager()->GetInteractiveToolsContext()->GetViewportInteractionsBehaviorSource();
		
		BehaviorSource->FindInteraction(USkeletalMeshModelingToolsGeometryFrustumSelection::StaticClass())->SetEnabled(bEnable);
		BehaviorSource->FindInteraction(USkeletalMeshModelingToolsGeometryClickSelection::StaticClass())->SetEnabled(bEnable);
		BehaviorSource->FindInteraction(USkeletalMeshModelingToolsGeometryDoubleClickSelection::StaticClass())->SetEnabled(bEnable);
		BehaviorSource->FindInteraction(USkeletalMeshModelingToolsGeometryHoverInteraction::StaticClass())->SetEnabled(bEnable);
		BehaviorSource->FindInteraction(USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::StaticClass())->SetEnabled(bEnable);
		
		BehaviorSource->FindInteraction(UViewportMoveYawInteraction::StaticClass())->SetEnabled(!bEnable);	
	}	
}


USkeletalMeshModelingToolsEditorMode::FSkeletonStateFactors USkeletalMeshModelingToolsEditorMode::GetSkeletonStateFactors()
{
	FSkeletonStateFactors Factors;
	Factors.bIsGeometrySelectionActive = IsGeometrySelectionActive();
	Factors.DesiredSkeletonState = DesiredSkeletonState;

	return Factors;
}

void USkeletalMeshModelingToolsEditorMode::CreateToolkit()
{
	TSharedPtr<FSkeletalMeshModelingToolsEditorModeToolkit> NewToolKit = MakeShareable(new FSkeletalMeshModelingToolsEditorModeToolkit);
	Toolkit = NewToolKit;
	TypedToolkit = NewToolKit;
}

bool USkeletalMeshModelingToolsEditorMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	if (OtherModeID == FPersonaEditModes::SkeletonSelection)
	{
		return true;
	}
	
	return Super::IsCompatibleWith(OtherModeID);
}

void USkeletalMeshModelingToolsEditorMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	Super::Tick(ViewportClient, DeltaTime);

	// Watchers fire causally from mutation sites; Tick only forwards per-frame work to the cache.
	if (GetCurrentEditingCache())
	{
		GetCurrentEditingCache()->Tick();
	}
}

bool USkeletalMeshModelingToolsEditorMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click)
{
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->HandleClick(HitProxy))
	{
		return true;
	}
	
	// Making sure we can still select bones from viewport even if FPersonaEditModes::SkeletonSelection is deactivated
	if (TSharedPtr<ISkeletalMeshEditorBinding> EditorBindingPtr = GetEditorBinding())
	{
		TArray<FName> Selected;
		if (HitProxy && EditorBindingPtr->GetNameFunction())
		{
			if (TOptional<FName> BoneName = EditorBindingPtr->GetNameFunction()(HitProxy))
			{
				Selected.Emplace(*BoneName);
			}
		}

		GetModeBinding()->GetNotifier()->HandleNotification(Selected, ESkeletalMeshNotifyType::BonesSelected);
	}
	
	return Super::HandleClick(InViewportClient, HitProxy, Click);
}

void USkeletalMeshModelingToolsEditorMode::PostUndo()
{
	Super::PostUndo();
}

bool USkeletalMeshModelingToolsEditorMode::OnRequestClose()
{
	if (GetCurrentEditingCache())
	{
		int32 ChangeCount = GetCurrentEditingCache()->GetEditingMeshComponent()->GetChangeCount();
		if (ChangeCount > 0 && GetSkeletalMesh())
		{
			EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
				FText::Format(
					LOCTEXT("Prompt_SkeletalMeshModelingToolsClose",
						"Skeletal Mesh \"{0}\" has {1} unapplied {1}|plural(one=change,other=changes). Would you like to apply before closing?"),
					FText::FromString(GetSkeletalMesh()->GetPathName()),
					ChangeCount));

			
			switch (Reply)
			{
			case EAppReturnType::Yes:
				{
					FScopedTransaction Transaction(LOCTEXT("ApplyChangesToAssetOnModeExit", "Apply Changes To Skeletal Mesh"));
					GetCurrentEditingCache()->ApplyChanges();
					return true;
				}
			case EAppReturnType::No:
				{
					FScopedTransaction Transaction(LOCTEXT("DiscardChangesOnModeExit", "Discard Pending Changes"));
					GetCurrentEditingCache()->DiscardChanges();
					return true;
				}
			case EAppReturnType::Cancel:
				return false;
			}
		}
	}
	return Super::OnRequestClose();
}

bool USkeletalMeshModelingToolsEditorMode::ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const
{
	// if Tool supports custom Focus box, use that first
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolCameraFocusAPI* FocusAPI = Cast<IInteractiveToolCameraFocusAPI>(Tool);
		if (FocusAPI && FocusAPI->SupportsWorldSpaceFocusBox())
		{
			InOutBox = FocusAPI->GetWorldSpaceFocusBox();
			return true;
		}
	}

	if (HasActiveGeometrySelection())
	{
		UE::Geometry::FGeometrySelectionBounds SelectionBounds;
		GetSelectionManager()->GetSelectionBounds(SelectionBounds);
		FBox FocusBox = (FBox)SelectionBounds.WorldBounds;
		const double MaxDimension = FocusBox.GetExtent().GetMax();
		InOutBox = FocusBox.ExpandBy(MaxDimension * 0.2);
		return true;
	}

	if (GetCurrentEditingCache())
	{
		TArray<FName> Selection = GetCurrentEditingCache()->GetSelectedBones();
	
		// focus using selected bones in skel mesh editor
		if (!Selection.IsEmpty())
		{
			const FReferenceSkeleton& RefSkeleton = GetCurrentEditingCache()->GetEditingMeshComponent()->GetRefSkeleton();
			TArray<FTransform> WorldTransforms = GetCurrentEditingCache()->GetComponentSpaceBoneTransforms();
			FTransform MeshTransform = GetCurrentEditingCache()->GetTransform();
			for (FTransform& Transform : WorldTransforms)
			{
				Transform *= MeshTransform;
			}
		
			TArray<FName> AllChildren;

			for (const FName& BoneName: Selection)
			{
				const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
				if (BoneIndex > INDEX_NONE && WorldTransforms.IsValidIndex(BoneIndex))
				{
					// enlarge box
					InOutBox += WorldTransforms[BoneIndex].GetLocation();

					// get direct children
					TArray<int32> Children;
					RefSkeleton.GetDirectChildBones(BoneIndex, Children);
					Algo::Transform(Children, AllChildren, [&RefSkeleton](int ChildrenIndex)
					{
						return RefSkeleton.GetBoneName(ChildrenIndex);
					});
				}
			}

			// enlarge box using direct children
			for (const FName& BoneName: AllChildren)
			{
				const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
				if (BoneIndex > INDEX_NONE && WorldTransforms.IsValidIndex(BoneIndex))
				{
					InOutBox += WorldTransforms[BoneIndex].GetLocation();	
				}
			}
		
			return true; 
		}
	}

	
	return Super::ComputeBoundingBoxForViewportFocus(Actor, PrimitiveComponent, InOutBox);
}

void USkeletalMeshModelingToolsEditorMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	// Tool action chord bindings go to a per-active-tool list (not the toolkit list) so chord
	// dispatch on InputKey can route through them without being short-circuited by a disabled
	// mode command sharing the chord. See InputKey override.
	ActiveToolCommandList = MakeShared<FUICommandList>();

	FSkeletalMeshModelingToolsActionCommands::UpdateToolCommandBinding(Tool, ActiveToolCommandList, false);
	if (TryGetExtensionToolCommandGetter(Manager, Tool, ExtensionToolCommandsGetter) && ensure(ExtensionToolCommandsGetter))
	{
		ExtensionToolCommandsGetter().BindCommandsForCurrentTool(ActiveToolCommandList, Tool);
	}
}

void USkeletalMeshModelingToolsEditorMode::RegisterTool(TSharedPtr<FUICommandInfo> UICommand, FString ToolIdentifier, UInteractiveToolBuilder* Builder, EToolsContextScope ToolScope)
{
	if (UICommand.IsValid() && !ToolIdentifier.IsEmpty())
	{
		ToolIdentifierToCommand.Add(ToolIdentifier, UICommand);
	}

	Super::RegisterTool(UICommand, MoveTemp(ToolIdentifier), Builder, ToolScope);
}

TSharedPtr<FUICommandInfo> USkeletalMeshModelingToolsEditorMode::FindCommandForTool(const FString& ToolIdentifier) const
{
	if (const TSharedPtr<FUICommandInfo>* Found = ToolIdentifierToCommand.Find(ToolIdentifier))
	{
		return *Found;
	}
	return nullptr;
}

void USkeletalMeshModelingToolsEditorMode::OnToolPostBuild( UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool,
	UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState)
{
	// Want to clear active selection when a Tool starts, but we have to wait until after it has been
	// built, so that the Tool has a chance to see the Selection
	if (GetSelectionManager() && GetSelectionManager()->HasSelection())
	{
		ensureMsgf(!GetSelectionManager()->HasSavedSelection(), TEXT("Selection manager should not already have a saved selection before we save-on-clear here in tool setup."));
		GetSelectionManager()->ClearSelection(/*bSaveSelectionBeforeClear*/ true);
	}
			
	if (GetCurrentEditingCache()->HasIsolation())
	{
		ensureMsgf(!GetCurrentEditingCache()->HasSavedIsolation(), TEXT("Editing Cache should not already have a saved isolation before we save-on-clear here in tool setup."));
		constexpr bool bSaveIsolation = true;
		GetCurrentEditingCache()->ShowFullMesh(bSaveIsolation);
	}
	
	// Tools can decide for themselves if they want to enable bone manipulation
	ToggleBoneManipulation(false);

	// Disable element selection modes while tool is active
	SetSuspendGeometrySelection(true);

	LastActivatedToolName = InToolManager->GetActiveToolName(InSide);
}

bool USkeletalMeshModelingToolsEditorMode::MouseEnter(FEditorViewportClient* /*ViewportClient*/, FViewport* /*Viewport*/, int32 /*X*/, int32 /*Y*/)
{
	// Don't steal focus while a popup menu is open — moving the cursor between menu items can briefly
	// cross into the viewport and would dismiss the menu otherwise.
	if (FSlateApplication::Get().AnyMenusVisible())
	{
		return false;
	}

	if (UPersonaEditorModeManagerContext* PersonaModeManagerContext = GetInteractiveToolsContext()->ContextObjectStore->FindContext<UPersonaEditorModeManagerContext>())
	{
		PersonaModeManagerContext->SetFocusInViewport();
	}
	return false;
}

void USkeletalMeshModelingToolsEditorMode::OnToolEndedWithStatus(UInteractiveToolManager* Manager, UInteractiveTool* Tool,
	EToolShutdownType ShutdownType)
{
	bool bRestoreEverything = (ShutdownType == EToolShutdownType::Cancel);
	bool bCanRestoreIsolation = false;
			
	if (ISkeletalMeshGeometryIsolationAwareTool* IsolateAwareTool = Cast<ISkeletalMeshGeometryIsolationAwareTool>(Tool))
	{
		bCanRestoreIsolation = bRestoreEverything || IsolateAwareTool->IsInputIsolationValidOnOutput();
	}
			
	if (bCanRestoreIsolation)
	{
		GetCurrentEditingCache()->RestoreSavedIsolation();	
	}
	else
	{
		GetCurrentEditingCache()->DiscardSavedIsolation();	
	}
	ensureMsgf(!GetCurrentEditingCache()->HasSavedIsolation(), TEXT("Editing Cache's saved isolation should be cleared on tool end."));
			

	bool bCanRestoreSelection = false;
	if (IInteractiveToolManageGeometrySelectionAPI* ManageSelectionTool = Cast<IInteractiveToolManageGeometrySelectionAPI>(Tool))
	{
		bCanRestoreSelection = bRestoreEverything || ManageSelectionTool->IsInputSelectionValidOnOutput();
	}
			
	if (bCanRestoreSelection)
	{
		GetSelectionManager()->RestoreSavedSelection();
	}
	else
	{
		GetSelectionManager()->DiscardSavedSelection();
	}
			
	ensureMsgf(!GetSelectionManager()->HasSavedSelection(), TEXT("Selection manager's saved selection should be cleared on tool end."));	
	
	if (ActiveToolCommandList.IsValid())
	{
		FSkeletalMeshModelingToolsActionCommands::UpdateToolCommandBinding(Tool, ActiveToolCommandList, true);
		if (ExtensionToolCommandsGetter)
		{
			ExtensionToolCommandsGetter().UnbindActiveCommands(ActiveToolCommandList);
		}
	}
	ExtensionToolCommandsGetter = nullptr;
	ActiveToolCommandList.Reset();
	
	ToggleBoneManipulation(true);

	SetSuspendGeometrySelection(false);

	if (bApplyChangesToAssetOnToolEnd)
	{
		bApplyChangesToAssetOnToolEnd = false;
		if (HasUnappliedChanges())
		{
			// Unfortunately this is currently a separate transaction because tools construct their own shutdown transaction
			// in different ways and there isn't a reliable place to inject this transaction into those transactions.
			FScopedTransaction Transaction(LOCTEXT("ApplyChangesToAssetOnToolEnd","Apply Changes To Skeletal Mesh"));
			ApplyChanges();
		}
	}
}

bool USkeletalMeshModelingToolsEditorMode::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if (Event != IE_Released)
	{
		const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		if (ActiveToolCommandList.IsValid()
			&& ActiveToolCommandList->ProcessCommandBindings(Key, ModifierKeys, Event == IE_Repeat))
		{
			return true;
		}
		if (IsGeometrySelectionActive()
			&& ActiveGeometrySelectionCommandList.IsValid()
			&& ActiveGeometrySelectionCommandList->ProcessCommandBindings(Key, ModifierKeys, Event == IE_Repeat))
		{
			return true;
		}
	}
	return Super::InputKey(ViewportClient, Viewport, Key, Event);
}

void USkeletalMeshModelingToolsEditorMode::OnToolContextRender(IToolsContextRenderAPI* ToolsContextRenderAPI)
{
	SelectionManager->DebugRender(ToolsContextRenderAPI);
}

void USkeletalMeshModelingToolsEditorMode::OnToolContextDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* ToolsContextRenderAPI)
{

}

bool USkeletalMeshModelingToolsEditorMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	// in the base mode, this returns false if the level editor is in PIE or simulated
	// we allow all skeletal mesh editing tools to be started while running in PIE / simulate
	return true;
}



bool USkeletalMeshModelingToolsEditorMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag,
	FRotator& InRot, FVector& InScale)
{
	const ECoordSystem CoordSystem = InViewportClient->GetWidgetCoordSystemSpace();	
	if (SelectionTransformTweaker->IsEditingTransform())
	{
		SelectionTransformTweaker->TweakTransform(InDrag, InRot, InScale, CoordSystem == COORD_World);
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->GetSkeletonPoser()->IsRecordingPoseChange())
	{
		// Actually change bone transform
		const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
		
		if (CurrentAxis != EAxisList::Type::None)
		{
			URefSkeletonPoser* Poser = GetCurrentEditingCache()->GetSkeletonPoser();

			int32 BoneIndex = GetCurrentEditingCache()->GetFirstSelectedBoneIndex();
			const FTransform& ComponentTransform = GetCurrentEditingCache()->GetTransform();
			const FTransform& ComponentSpaceTransform = Poser->GetComponentSpaceTransform(BoneIndex);
			FTransform BoneGlobal = ComponentSpaceTransform * ComponentTransform;

			FTransform BaseTransform = BoneGlobal;
			if (TOptional<FTransform> Additive = Poser->GetBoneAdditiveTransform(BoneIndex))
			{
				BaseTransform = BoneGlobal.GetRelativeTransformReverse(*Additive);
			}

			Poser->ModifyBoneAdditiveTransform(BoneIndex,
				[&BaseTransform, &InDrag, &InRot, &InScale, &CoordSystem](FTransform& InTransform)
				{
					FVector Offset = BaseTransform.TransformVector(InDrag);

					FVector RotAxis;
					float RotAngle;
					InRot.Quaternion().ToAxisAndAngle(RotAxis, RotAngle);

					FVector4 BoneSpaceAxis = BaseTransform.TransformVectorNoScale(RotAxis);

					//Calculate the new delta rotation
					FQuat DeltaQuat(BoneSpaceAxis, RotAngle);
					DeltaQuat.Normalize();

					FQuat NewRotation = (InTransform * FTransform(DeltaQuat)).GetRotation();

					FVector4 BoneSpaceScaleOffset;

					if (CoordSystem == COORD_World)
					{
						BoneSpaceScaleOffset = BaseTransform.TransformVector(InScale);
					}
					else
					{
						BoneSpaceScaleOffset = InScale;
					}

					InTransform.SetTranslation(InTransform.GetTranslation() + Offset);
					InTransform.SetRotation(NewRotation);
					InTransform.SetScale3D(InTransform.GetScale3D() + BoneSpaceScaleOffset);
				});

			return true;
		}	
	}

	return false;
}

void USkeletalMeshModelingToolsEditorMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View,
	FCanvas* Canvas)
{
	if (GetCurrentEditingCache())
	{
		// Draw bone name
		GetCurrentEditingCache()->DrawHUD(ViewportClient, Viewport, View, Canvas);
	}

	DrawActiveToolHotkeyHints(Viewport, Canvas);
}

void USkeletalMeshModelingToolsEditorMode::DrawActiveToolHotkeyHints(FViewport* Viewport, FCanvas* Canvas) const
{
	if (Canvas == nullptr || Viewport == nullptr)
	{
		return;
	}

	UInteractiveToolManager* ToolManager = GetToolManager();
	UInteractiveTool* ActiveTool = ToolManager ? ToolManager->GetActiveTool(EToolSide::Left) : nullptr;

	using FHotkeyHint = IHotkeyHintProvider::FHotkeyHint;
	TArray<FHotkeyHint> Hints;
	
	// Give Quick Access Menu the priority so users are aware they can switch to geo mode even if a tool is active
	{
		const FSkeletalMeshModelingToolsCommands& ModeCommands = FSkeletalMeshModelingToolsCommands::Get();
		const TSharedPtr<FUICommandList> ToolkitCommandList = Toolkit.IsValid() ? Toolkit->GetToolkitCommands().ToSharedPtr(): nullptr;
		IHotkeyHintProvider::TryAppendCommandHint(Hints, ModeCommands.ShowQuickAccessMenu, ToolkitCommandList);
	}

	// Mesh edges rendering is also a frequently used command, surface to users in all contexts
	if (const TSharedPtr<const FUICommandInfo> MeshEdgesCmd =
			FShowFlagMenuCommands::Get().FindCommandForFlag(FEngineShowFlags::EShowFlag::SF_MeshEdges))
	{
		Hints.Add({ LOCTEXT("HintToggleMeshEdgeRendering", "Toggle Mesh Edges Rendering"), MeshEdgesCmd->GetInputText() });
	}
	
	if (ActiveTool)
	{
		if (const IHotkeyHintProvider* Provider = Cast<IHotkeyHintProvider>(ActiveTool))
		{
			Provider->GetHotkeyHints(Hints);
		}
	}
	else
	{
		GetHotkeyHints(Hints);
	}
	
	const float DPIInvScale = Canvas->GetDPIScale() > 0.f ? (1.f / Canvas->GetDPIScale()) : 1.f;
	constexpr int32 RightMargin = 20;
	constexpr int32 TopInset    = 20;
	constexpr int32 RowPadding  = 4;

	const int32 ViewportWidth = static_cast<int32>(Viewport->GetSizeXY().X * DPIInvScale);

	// Two text items per row so chord can be tinted (matches MetaHuman's SetShortcuts panel).
	// Drawing transparent first to populate DrawnSize, then drawing again at the resolved position.
	// Title is split into a "Hotkeys: " label and the toggle chord so the chord can pick up the
	// same AccentBlue tint as the per-row chord values below, matching the row label format.
	FText TitleLabelText = LOCTEXT("HotkeysTitle", "Hotkeys");
	FText TitleChordText;
	if (const TSharedPtr<FUICommandInfo> ToggleCmd = FSkeletalMeshModelingToolsCommands::Get().ToggleHotkeyHints)
	{
		const TSharedRef<const FInputChord> ToggleChord = ToggleCmd->GetActiveChord(EMultipleKeyBindingIndex::Primary);
		if (ToggleChord->IsValidChord())
		{
			TitleLabelText = FText::Format(LOCTEXT("HotkeysTitleLabelWithChord", "{0}: "), TitleLabelText);
			TitleChordText = ToggleChord->GetInputText();
		}
	}
	FCanvasTextItem TitleLabelItem(FVector2D::ZeroVector, TitleLabelText,
		GEngine->GetLargeFont(), FLinearColor::Transparent);
	Canvas->DrawItem(TitleLabelItem);
	const float TitleLabelWidth = TitleLabelItem.DrawnSize.X * DPIInvScale;

	FCanvasTextItem TitleChordItem(FVector2D::ZeroVector, TitleChordText,
		GEngine->GetLargeFont(), FLinearColor::Transparent);
	float TitleChordWidth = 0.f;
	if (!TitleChordText.IsEmpty())
	{
		Canvas->DrawItem(TitleChordItem);
		TitleChordWidth = TitleChordItem.DrawnSize.X * DPIInvScale;
	}
	float MaxRowWidth = TitleLabelWidth + TitleChordWidth;

	TArray<FCanvasTextItem> LabelItems;
	TArray<FCanvasTextItem> ValueItems;
	TArray<float> LabelWidths;
	if (bShowHotkeyHints)
	{
		LabelItems.Reserve(Hints.Num());
		ValueItems.Reserve(Hints.Num());
		LabelWidths.Reserve(Hints.Num());
		for (const FHotkeyHint& Hint : Hints)
		{
			const FText LabelText = FText::Format(LOCTEXT("HotkeyHintLabelFormat", "{0}: "), Hint.Label);
			FCanvasTextItem& LabelItem = LabelItems.Emplace_GetRef(FVector2D::ZeroVector, LabelText,
				GEngine->GetSmallFont(), FLinearColor::Transparent);
			FCanvasTextItem& ValueItem = ValueItems.Emplace_GetRef(FVector2D::ZeroVector, Hint.HotkeyText,
				GEngine->GetSmallFont(), FLinearColor::Transparent);

			Canvas->DrawItem(LabelItem);
			Canvas->DrawItem(ValueItem);
			const float LabelWidth = LabelItem.DrawnSize.X * DPIInvScale;
			const float ValueWidth = ValueItem.DrawnSize.X * DPIInvScale;
			LabelWidths.Add(LabelWidth);
			MaxRowWidth = FMath::Max(MaxRowWidth, LabelWidth + ValueWidth);
		}
	}

	const int32 LeftEdgeX = static_cast<int32>(ViewportWidth - RightMargin - MaxRowWidth);
	int32 Y = TopInset;

	const FLinearColor ChordColor = FStyleColors::AccentBlue.GetSpecifiedColor();

	TitleLabelItem.Position = FVector2D(LeftEdgeX, Y);
	TitleLabelItem.SetColor(FLinearColor::White);
	Canvas->DrawItem(TitleLabelItem);
	const float TitleRowHeight = TitleLabelItem.DrawnSize.Y * DPIInvScale;

	if (!TitleChordText.IsEmpty())
	{
		TitleChordItem.Position = FVector2D(LeftEdgeX + TitleLabelWidth, Y);
		TitleChordItem.SetColor(ChordColor);
		Canvas->DrawItem(TitleChordItem);
	}
	Y += TitleRowHeight + RowPadding * 2;

	if (!bShowHotkeyHints)
	{
		return;
	}

	for (int32 Index = 0; Index < LabelItems.Num(); ++Index)
	{
		FCanvasTextItem& LabelItem = LabelItems[Index];
		FCanvasTextItem& ValueItem = ValueItems[Index];

		LabelItem.Position = FVector2D(LeftEdgeX, Y);
		LabelItem.SetColor(FLinearColor::White);
		Canvas->DrawItem(LabelItem);

		ValueItem.Position = FVector2D(LeftEdgeX + LabelWidths[Index], Y);
		ValueItem.SetColor(ChordColor);
		Canvas->DrawItem(ValueItem);

		Y += LabelItem.DrawnSize.Y * DPIInvScale + RowPadding;
	}
}

void USkeletalMeshModelingToolsEditorMode::GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const
{
	const FSkeletalMeshModelingToolsCommands& ModeCommands = FSkeletalMeshModelingToolsCommands::Get();
	const TSharedPtr<FUICommandList> ToolkitCommandList = Toolkit.IsValid() ? Toolkit->GetToolkitCommands().ToSharedPtr(): nullptr;


	if (IsGeometrySelectionActive())
	{
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.BeginSelectionAction_SelectAll, ToolkitCommandList);
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.BeginSelectionAction_Invert, ToolkitCommandList);
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.BeginSelectionAction_InvertConnected, ToolkitCommandList);
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.BeginSelectionAction_Expand, ToolkitCommandList);
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.BeginSelectionAction_Contract, ToolkitCommandList);

		OutHints.Add({ LOCTEXT("HintSelectIsland", "Select Connected Island"),
			LOCTEXT("HintSelectIslandChord", "Double-click") });
		OutHints.Add({ LOCTEXT("HintSoftSelectRadius", "Soft Selection Radius"),
			LOCTEXT("HintSoftSelectRadiusChord", "B + drag left/right") });

		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.IsolateSelection, ToolkitCommandList);
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.HideSelection, ToolkitCommandList);
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.ShowFullMesh, ToolkitCommandList);

		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ModeCommands.MirrorEditingMorphTarget, ToolkitCommandList);
	}
	else if (GetEditor())
	{
		const TSharedPtr<FUICommandList> EditorCommandList = GetEditor()->GetToolkitCommands();
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, GetEditor()->GetResetBoneTransformsCommand(), EditorCommandList);
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, GetEditor()->GetResetAllBonesTransformsCommand(), EditorCommandList);
	}
}

bool USkeletalMeshModelingToolsEditorMode::AllowWidgetMove()
{
	if (HasActiveGeometrySelection())
	{
		return true;
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->IsDynamicMeshSkeletonEnabled())
	{
		return ShouldDrawWidget();
	}
	return Super::AllowWidgetMove();
}

bool USkeletalMeshModelingToolsEditorMode::ShouldDrawWidget() const
{
	if (HasActiveGeometrySelection())
	{
		if (GetEditingMorphTarget() == NAME_None)
		{
			return bAllowEditingBaseMeshViaGeometrySelection;
		}
			
		return true;
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->IsDynamicMeshSkeletonEnabled())
	{
		if (GetCurrentEditingCache()->IsDynamicMeshBoneManipulationEnabled() && GetCurrentEditingCache()->GetFirstSelectedBoneIndex() != INDEX_NONE )
		{
			return true;
		}

		return false;
	}

	return Super::ShouldDrawWidget();
}

bool USkeletalMeshModelingToolsEditorMode::UsesTransformWidget() const
{
	// This function determines if the GetWidgetLocation() of this editor mode should be used
	if (HasActiveGeometrySelection())
	{
		return true;
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->IsDynamicMeshSkeletonEnabled())
	{
		return true;
	}
	
	return false;
}

FVector USkeletalMeshModelingToolsEditorMode::GetWidgetLocation() const
{
	if (HasActiveGeometrySelection())
	{
		return SelectionTransformTweaker->GetSelectionFrameTransform().GetLocation();
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->IsDynamicMeshSkeletonEnabled())
	{
		int32 BoneIndex = GetCurrentEditingCache()->GetFirstSelectedBoneIndex();
		URefSkeletonPoser* Poser = GetCurrentEditingCache()->GetSkeletonPoser();
		FTransform ComponentTransform = GetCurrentEditingCache()->GetTransform();
		if (BoneIndex != INDEX_NONE)
		{
			FTransform ComponentSpaceTransform = Poser->GetComponentSpaceTransform(BoneIndex);
		
			FTransform WorldSpaceTransform = ComponentSpaceTransform * ComponentTransform;

			return WorldSpaceTransform.GetLocation();
		}
	
		return ComponentTransform.GetLocation();
	}

	return Super::GetWidgetLocation();
}



bool USkeletalMeshModelingToolsEditorMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	if (HasActiveGeometrySelection())
	{
		InMatrix = SelectionTransformTweaker->GetSelectionFrameTransform().ToMatrixNoScale().RemoveTranslation();
		return true;
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->IsDynamicMeshSkeletonEnabled())
	{
		const bool bIsParentMode = Owner ? Owner->GetCoordSystem() == COORD_Parent : false;
	
		const FReferenceSkeleton& RefSkeleton = GetCurrentEditingCache()->GetEditingMeshComponent()->GetRefSkeleton();
		int32 BoneIndex = GetCurrentEditingCache()->GetFirstSelectedBoneIndex();
		URefSkeletonPoser* Poser = GetCurrentEditingCache()->GetSkeletonPoser();
		FTransform ComponentTransform = GetCurrentEditingCache()->GetTransform();
		if (RefSkeleton.IsValidIndex(BoneIndex))
		{
			if (bIsParentMode)
			{
				const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					BoneIndex = ParentIndex;
				}
			}
		
			FTransform BoneGlobalTransform = Poser->GetComponentSpaceTransform(BoneIndex) * ComponentTransform;	

			InMatrix = BoneGlobalTransform.ToMatrixNoScale().RemoveTranslation();
			return true;
		}

		return false;
	}
	
	return Super::GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

bool USkeletalMeshModelingToolsEditorMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	if (HasActiveGeometrySelection())
	{
		return GetCustomDrawingCoordinateSystem(InMatrix, InData);	
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->IsDynamicMeshSkeletonEnabled())
	{
		return GetCustomDrawingCoordinateSystem(InMatrix, InData);
	}

	return Super::GetCustomInputCoordinateSystem(InMatrix, InData);
}

USkeletalMeshBackedDynamicMeshComponent* USkeletalMeshModelingToolsEditorMode::GetComponent(UObject* SourceObject)
{
	if (!GetCurrentEditingCache())
	{
		return nullptr;
	}
	
	return GetCurrentEditingCache()->GetEditingMeshComponent();
}

void USkeletalMeshModelingToolsEditorMode::RequestApplyChangesToAssetOnToolEnd()
{
	bApplyChangesToAssetOnToolEnd = true;
}


bool USkeletalMeshModelingToolsEditorMode::BeginTransform(const FGizmoState& InState)
{
	if (HasActiveGeometrySelection())
	{
		SelectionTransformTweaker->BeginTransformEdit();
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->IsDynamicMeshBoneManipulationEnabled())
	{
		if (GetCurrentEditingCache()->GetFirstSelectedBoneIndex() != INDEX_NONE)
		{
			GetCurrentEditingCache()->GetSkeletonPoser()->BeginPoseChange();
			return true;
		}
	}
	
	return false;
}

bool USkeletalMeshModelingToolsEditorMode::EndTransform(const FGizmoState& InState)
{
	if (SelectionTransformTweaker->IsEditingTransform())
	{
		SelectionTransformTweaker->EndTransformEdit();

		return true;
	}
	
	if (GetCurrentEditingCache() && GetCurrentEditingCache()->GetSkeletonPoser()->IsRecordingPoseChange())
	{
		GetCurrentEditingCache()->GetSkeletonPoser()->EndPoseChange();
		
		return true;
	}
	return false;	
}


TSharedPtr<ISkeletalMeshEditorBinding> USkeletalMeshModelingToolsEditorMode::GetModeBinding()
{
	if (!ModeBinding.IsValid())
	{
		ModeBinding = MakeShared<FSkeletalMeshModelingToolsEditorModeBinding>(this);
	}

	return ModeBinding;
}

void USkeletalMeshModelingToolsEditorMode::SetEditorBinding(const TWeakPtr<ISkeletalMeshEditor>& InSkeletalMeshEditor)
{
	if (!InSkeletalMeshEditor.IsValid())
	{
		return;
	}
	
	EditorBinding = InSkeletalMeshEditor.Pin()->GetBinding();
	Editor = InSkeletalMeshEditor;

	Editor.Pin()->OnPreSaveAsset().AddUObject(this, &USkeletalMeshModelingToolsEditorMode::HandlePreSaveAsset);
	Editor.Pin()->OnPreSaveAssetAs().AddUObject(this, &USkeletalMeshModelingToolsEditorMode::HandlePreSaveAsset);

	SkeletalMesh = InSkeletalMeshEditor.Pin()->GetPersonaToolkit()->GetMesh();
	SkeletalMesh->GetOnPreMeshChange().AddUObject(this, &USkeletalMeshModelingToolsEditorMode::HandleSkeletalMeshPreChange);
	SkeletalMesh->GetOnMeshChanged().AddUObject(this, &USkeletalMeshModelingToolsEditorMode::HandleSkeletalMeshChanged);
	SkeletonReader = NewObject<USkeletonModifier>();
	SkeletonReader->SetReferenceSkeleton(SkeletalMesh->GetRefSkeleton());
	SkeletonReader->SetReadOnly(true);
	
	SelectedBones = EditorBinding.Pin()->GetSelectedBones();
	
	if (USkeletalMeshEditorContextObject* ContextObject = UE::SkeletalMeshEditorUtils::GetEditorContextObject(GetInteractiveToolsContext()))
	{
		ContextObject->Init(this);
	}
	
	OnInitializedDelegate.Broadcast();
}

TSharedPtr<ISkeletalMeshEditorBinding> USkeletalMeshModelingToolsEditorMode::GetEditorBinding()
{
	if (!EditorBinding.IsValid())
	{
		return nullptr;
	}
	
	return EditorBinding.Pin();
}

TSharedPtr<ISkeletalMeshEditor> USkeletalMeshModelingToolsEditorMode::GetEditor() const
{
	if (!Editor.IsValid())
	{
		return nullptr;
	}

	return Editor.Pin();
}

USkeletalMesh* USkeletalMeshModelingToolsEditorMode::GetSkeletalMesh() const
{
	return SkeletalMesh.Get();
}

void USkeletalMeshModelingToolsEditorMode::HandleSkeletalMeshPreChange()
{
	if (GetCurrentEditingCache())
	{
		// External change detected
		if (!GetCurrentEditingCache()->GetEditingMeshComponent()->IsExpectingAssetChange())
		{
			// Apply pending changes before applying external changes on top
			if (HasUnappliedChanges())
			{
				bExternalChangeDetected = true;
				FScopedTransaction Transaction(LOCTEXT("ApplyPendingBeforeExternalChange", "Apply Pending Changes To Skeletal Mesh"));
				GetCurrentEditingCache()->ApplyChanges();
			}
		}
	}
}

void USkeletalMeshModelingToolsEditorMode::HandleSkeletalMeshChanged()
{
	USkeletalMeshBackedDynamicMeshComponent* EditingMeshComponent = GetCurrentEditingCache() ? GetCurrentEditingCache()->GetEditingMeshComponent() : nullptr;
	if (!EditingMeshComponent)
	{
		return;
	}

	if (EditingMeshComponent->IsExpectingAssetChange())
	{
		EditingMeshComponent->ClearExpectAssetChange();
		
		// If cache change is not part of an external change, no need to recreate the cache
		if (!bExternalChangeDetected)
		{
			// Simply refresh Skeletal Mesh Skeleton Tree
			GetEditorBinding()->GetNotifier()->HandleNotification({}, ESkeletalMeshNotifyType::HierarchyChanged);
			GetEditorBinding()->GetNotifier()->HandleNotification(GetSelectedBones(), ESkeletalMeshNotifyType::BonesSelected);
			return;
		}
	}
	
	// Recreating Cache, undo history will be invalidated
	int32 ChangeCount = EditingMeshComponent->GetChangeCount();
	if (bExternalChangeDetected)
	{
		// Changes should have been applied during PreChange
		ensure(ChangeCount == 0);
	}
	else
	{
		// Unexpected external change, need to sync editing cache to 
		// the changed asset
		if (ChangeCount > 0)
		{
			FText Message = FText::Format(LOCTEXT("ExternalAssetChangeDiscardPendingToolChanges", "{0} change(s) discarded due to untracked external asset change"), ChangeCount);
			GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserWarning);
			ShowEditorMessage(ELogVerbosity::Type::Warning, Message);
		}
	}
	
	RecreateEditingCache(GetEditingLOD());
	
	bExternalChangeDetected = false;
}

bool USkeletalMeshModelingToolsEditorMode::CanSetEditingLOD()
{
	return !GetToolManager()->HasAnyActiveTool();
}

void USkeletalMeshModelingToolsEditorMode::SetEditingLOD(EMeshLODIdentifier InEditingLOD)
{
	if (GetCurrentEditingCache() && HasUnappliedChanges())
	{
		FScopedTransaction Transaction(LOCTEXT("ApplyChangesBeforeLODChange", "Apply Changes Before LOD Change"));
		GetCurrentEditingCache()->ApplyChanges();
	}
	RecreateEditingCache(InEditingLOD);

	if (GetCurrentEditingCache())
	{
		SkeletonReader->ExternalUpdate(GetCurrentEditingCache()->GetEditingMeshComponent()->GetRefSkeleton());

		TObjectPtr<UToolTargetManager> TargetManager = GetInteractiveToolsContext(EToolsContextScope::EdMode)->TargetManager;
		if (USkeletalMeshComponentToolTargetFactory* SkeletalMeshTargetFactory = TargetManager->FindFirstFactoryByType<USkeletalMeshComponentToolTargetFactory>())
		{
			SkeletalMeshTargetFactory->SetActiveEditingLOD(InEditingLOD);
		}
	}
}

EMeshLODIdentifier USkeletalMeshModelingToolsEditorMode::GetEditingLOD()
{
	if (!GetCurrentEditingCache())
	{
		return EMeshLODIdentifier::LOD0;
	}
	return GetCurrentEditingCache()->GetLOD();
}

void USkeletalMeshModelingToolsEditorMode::RecreateEditingCache(EMeshLODIdentifier InLOD)
{
	SelectionManager->ClearActiveTargets();
	
	if (CurrentEditingCache)
	{
		CurrentEditingCache->Destroy();
		EditingCacheNotifierBindScope.Reset();
	}
	
	if (!Editor.IsValid())
	{
		return;
	}

	if (!ensure(GetSkelMeshComponent()) || !ensure(GetSkelMeshComponent()->GetSkeletalMeshAsset()))
	{
		UE_LOGF(LogSkeletalMeshModelingTools, Error, "Failed to create editing cache for Skeletal Mesh");
		return;
	}
	
	TSharedRef<IPersonaPreviewScene> PreviewScene = Editor.Pin()->GetPersonaToolkit()->GetPreviewScene();
	UWorld* PreviewWorld = PreviewScene->GetWorld();
	
	CurrentEditingCache = NewObject<USkeletalMeshEditingCache>();

	
	USkeletalMeshEditingCache::FDelegates Delegates;
	Delegates.ToggleSkeletalMeshBoneManipulationDelegate.BindUObject(this, &USkeletalMeshModelingToolsEditorMode::ToggleSkeletalMeshBoneManipulation);
	Delegates.IsSkeletalMeshBoneManipulationEnabledDelegate.BindUObject(this, &USkeletalMeshModelingToolsEditorMode::IsSkeletalMeshBoneManipulationEnabled);
	
	Delegates.OnGetSkeletalMeshSkeletonNotifierDelegate.BindLambda([WeakThis = TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode>(this)]()
		{
			// Skeletal Mesh Skeleton bone selection state is governed by the skeleton tree in the base editor
			if (WeakThis.IsValid())
			{
				return WeakThis->GetEditorBinding()->GetNotifier();
			}

			return TSharedPtr<ISkeletalMeshNotifier>();
		});

	Delegates.OnComponentChangedEvent.AddUObject(this, &USkeletalMeshModelingToolsEditorMode::HandleEditingCacheComponentChanged);
	Delegates.OnSkeletonChangedEvent.AddUObject(this, &USkeletalMeshModelingToolsEditorMode::HandleEditingCacheSkeletonChanged);
	Delegates.OnPreviewMeshDeformedEvent.AddUObject(this, &USkeletalMeshModelingToolsEditorMode::HandleEditingCachePreviewMeshDeformed);

	
	CurrentEditingCache->Spawn(PreviewWorld, GetSkelMeshComponent(), InLOD, Delegates, GetToolManager()->GetContextTransactionsAPI());

	EditingCacheNotifierBindScope.Reset(new FSkeletalMeshNotifierBindScope(GetModeBinding()->GetNotifier(), CurrentEditingCache->GetNotifier()));

	CurrentEditingCache->GetNotifier()->HandleNotification({}, ESkeletalMeshNotifyType::HierarchyChanged);
	CurrentEditingCache->GetNotifier()->HandleNotification(GetSelectedBones(), ESkeletalMeshNotifyType::BonesSelected);
	OnMorphTargetDataChangedDelegate.Broadcast();
	
	
	SelectionManager->AddActiveTarget(FGeometryIdentifier::PrimitiveComponent(CurrentEditingCache->GetEditingMeshComponent(), FGeometryIdentifier::EObjectType::DynamicMeshComponent));


}



USkeletalMeshEditingCache* USkeletalMeshModelingToolsEditorMode::GetCurrentEditingCache() const
{
	return CurrentEditingCache;
}

bool USkeletalMeshModelingToolsEditorMode::HasUnappliedChanges() const
{
	if (!GetCurrentEditingCache())
	{
		return false;
	}
	
	return GetCurrentEditingCache()->GetEditingMeshComponent()->IsDirty();
}

void USkeletalMeshModelingToolsEditorMode::ApplyChanges()
{
	if (!GetCurrentEditingCache())
	{
		return;
	}
	GetCurrentEditingCache()->ApplyChanges();
}

void USkeletalMeshModelingToolsEditorMode::HandlePreSaveAsset()
{
	// Note: AutoSave should not trigger this function because it bypasses 
	// FSkeletalMeshEditor::SaveAsset_Execute and never broadcasts OnPreSaveAsset,
	// which happens to match the desired behavior because we don't want autosave to interrupt users while they are using a tool either.
	// If we do want autosave to also apply pending changes, we likely need to subscribe to a different delegate, 

	if (!GetCurrentEditingCache())
	{
		return;
	}

	UInteractiveToolManager* ToolManager = GetInteractiveToolsContext()->ToolManager;
	if (!ToolManager->HasActiveTool(EToolSide::Left) && !HasUnappliedChanges())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ApplyChangesOnSave", "Apply Changes To Skeletal Mesh Before Save"));
	FToolReactivateScope ReactivateScope(this);
	ApplyChanges();
}

void USkeletalMeshModelingToolsEditorMode::DiscardChanges()
{
	if (!GetCurrentEditingCache())
	{
		return;
	}
	
	GetCurrentEditingCache()->DiscardChanges();
}

USkeletalMeshModelingToolsEditorMode::EToolAcceptAction USkeletalMeshModelingToolsEditorMode::GetToolAcceptAction() const
{
	return ToolAcceptAction;
}

void USkeletalMeshModelingToolsEditorMode::SetToolAcceptAction(EToolAcceptAction InAction)
{
	ToolAcceptAction = InAction;
}

void USkeletalMeshModelingToolsEditorMode::HideSkeletonForTool()
{
	DesiredSkeletonState.bVisible = false;
	SkeletonStateUpdater.CheckAndUpdate();
}

void USkeletalMeshModelingToolsEditorMode::ShowSkeletonForTool()
{
	DesiredSkeletonState.bVisible = true;
	SkeletonStateUpdater.CheckAndUpdate();
}

ISkeletalMeshEditingInterface* USkeletalMeshModelingToolsEditorMode::GetSkeletonInterface(UInteractiveTool* InTool)
{
	if (!IsValid(InTool) || !InTool->Implements<USkeletalMeshEditingInterface>())
	{
		return nullptr;
	}
	return static_cast<ISkeletalMeshEditingInterface*>(InTool->GetInterfaceAddress(USkeletalMeshEditingInterface::StaticClass()));
}


void USkeletalMeshModelingToolsEditorMode::ToggleSkeletalMeshBoneManipulation(bool bEnable)
{
	if (Owner)
	{
		if(bEnable)
		{
			Owner->ActivateMode(FPersonaEditModes::SkeletonSelection);
		}
		else
		{
			Owner->DeactivateMode(FPersonaEditModes::SkeletonSelection);
		}
	}
}

bool USkeletalMeshModelingToolsEditorMode::IsSkeletalMeshBoneManipulationEnabled()
{
	if (Owner)
	{
		return Owner->IsModeActive(FPersonaEditModes::SkeletonSelection);
	}

	return true;
}


// --- Geometry Isolation ---

bool USkeletalMeshModelingToolsEditorMode::IsolateSelection()
{
	if (!CurrentEditingCache || !SelectionManager || !SelectionManager->HasSelection())
	{
		return false;
	}

	UPrimitiveComponent* EditingComponent = CurrentEditingCache->GetEditingMeshComponent();
	UE::Geometry::FGeometrySelection Selection;
	SelectionManager->GetSelectionForComponent(EditingComponent, Selection);

	return CurrentEditingCache->IsolateSelection(Selection);
}

bool USkeletalMeshModelingToolsEditorMode::HideSelection()
{
	if (!CurrentEditingCache || !SelectionManager || !SelectionManager->HasSelection())
	{
		return false;
	}

	UPrimitiveComponent* EditingComponent = CurrentEditingCache->GetEditingMeshComponent();
	UE::Geometry::FGeometrySelection Selection;
	SelectionManager->GetSelectionForComponent(EditingComponent, Selection);

	return CurrentEditingCache->HideSelection(Selection);
}

bool USkeletalMeshModelingToolsEditorMode::ShowFullMesh()
{
	if (!CurrentEditingCache || !SelectionManager)
	{
		return false;
	}

	return CurrentEditingCache->ShowFullMesh();
}

void USkeletalMeshModelingToolsEditorMode::ShowQuickAccessMenu()
{
	if (!Toolkit.IsValid())
	{
		return;
	}

	const TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> WeakThis(this);
	UInteractiveToolManager* ToolManager = GetToolManager();
	const bool bToolActive = ToolManager && ToolManager->HasActiveTool(EToolSide::Left);

	auto Populate = [WeakThis, bToolActive](FMenuBuilder& MenuBuilder)
	{
		if (!WeakThis.IsValid())
		{
			return;
		}

		const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();

		MenuBuilder.BeginSection("SelectionMode", LOCTEXT("QuickAccessSelectionSection", "Selection"));
		MenuBuilder.AddMenuEntry(Commands.MeshSelectionModeAction_NoSelection);
		MenuBuilder.AddMenuEntry(Commands.MeshSelectionModeAction_MeshVertices);
		MenuBuilder.AddMenuEntry(Commands.MeshSelectionModeAction_MeshEdges);
		MenuBuilder.AddMenuEntry(Commands.MeshSelectionModeAction_MeshTriangles);
		MenuBuilder.EndSection();

		if (!bToolActive && !WeakThis->LastActivatedToolName.IsEmpty())
		{
			if (const TSharedPtr<FUICommandInfo>* ToolCommand = WeakThis->ToolIdentifierToCommand.Find(WeakThis->LastActivatedToolName))
			{
				if (ToolCommand->IsValid())
				{
					MenuBuilder.BeginSection("LastTool", LOCTEXT("QuickAccessLastToolSection", "Recently Used"));
					MenuBuilder.AddMenuEntry(
						(*ToolCommand)->GetLabel(),
						(*ToolCommand)->GetDescription(),
						(*ToolCommand)->GetIcon(),
						FUIAction(FExecuteAction::CreateLambda([WeakThis, ToolCommandRef = ToolCommand->ToSharedRef()]()
						{
							if (WeakThis.IsValid())
							{
								WeakThis->Toolkit->GetToolkitCommands()->ExecuteAction(ToolCommandRef);
							}
						}),
						FCanExecuteAction::CreateLambda([WeakThis, ToolCommandRef = ToolCommand->ToSharedRef()]()
						{
							return WeakThis.IsValid() && WeakThis->Toolkit->GetToolkitCommands()->CanExecuteAction(ToolCommandRef);
						})),
						NAME_None,
						EUserInterfaceActionType::Button);
					MenuBuilder.EndSection();
				}
			}
		}
	};

	TSharedRef<SSkeletalMeshModelingToolsQuickAccessMenu> Menu =
		SNew(SSkeletalMeshModelingToolsQuickAccessMenu)
			.CommandList(Toolkit->GetToolkitCommands())
			.OnPopulateMenu_Lambda(Populate);

	TSharedPtr<SWidget> ParentWidget = FSlateApplication::Get().GetUserFocusedWidget(0);
	if (!ParentWidget.IsValid())
	{
		ParentWidget = FGlobalTabmanager::Get()->GetRootWindow();
	}

	if (!ParentWidget.IsValid())
	{
		return;
	}

	FSlateApplication::Get().PushMenu(
		ParentWidget.ToSharedRef(),
		FWidgetPath(),
		Menu,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

#undef LOCTEXT_NAMESPACE
