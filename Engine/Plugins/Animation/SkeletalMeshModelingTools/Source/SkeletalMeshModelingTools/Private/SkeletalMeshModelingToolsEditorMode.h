// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdMode.h"
#include "GizmoEdModeInterface.h"
#include "ModelingSelectionInteraction.h"
#include "MorphTargetManagerDataSource.h"
#include "SkeletalMeshModelingModeToolExtensions.h"
#include "SkeletalMeshNotifier.h"
#include "ToolContextInterfaces.h"
#include "Interfaces/ISKMBackedDynaMeshComponentProvider.h"
#include "SkeletalMesh/IHotkeyHintProvider.h"
#include "Changes/ValueWatcher.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

#include "SkeletalMeshModelingToolsEditorMode.generated.h"

class UViewportInteractionsBehaviorSource;
class USkeletalMeshGeometrySelectionTransformTweaker;
class FSkeletalMeshModelingToolsEditorModeToolkit;

class USkeletalMeshEditingCache;
//class FStylusStateTracker;
class UEdModeInteractiveToolsContext;
class ISkeletalMeshNotifier;
class ISkeletalMeshEditorBinding;
class ISkeletalMeshEditingInterface;
class HHitProxy;
class UDebugSkelMeshComponent;
class ISkeletalMeshEditor;
enum class EToolManagerToolSwitchMode;
class FTabManager;
enum class EToolSide;
enum class EToolShutdownType : uint8;
struct FToolBuilderState;
class USkeletonModifier;
class UGeometrySelectionManager;
class UModelingSelectionInteraction;
class UInteractiveCommand;
class USkeletalMesh;

namespace UE
{
	class IInteractiveToolCommandsInterface;
}

class USkeletalMeshModelingToolsEditorMode;

// Mode Notifier simply serves as middle man between tools and editing cache to help
// record bone selection during tool <-> cache transitions, it does not start notifications
// instead it simply forwards the notification
class FSkeletalMeshModelingToolsEditorModeNotifier: public ISkeletalMeshNotifier
{
public:
	FSkeletalMeshModelingToolsEditorModeNotifier(USkeletalMeshModelingToolsEditorMode* InEditorMode);
	virtual void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) override;
private:
	TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> EditorMode;
};

class FSkeletalMeshModelingToolsEditorModeBinding: public ISkeletalMeshEditorBinding
{
public:
	FSkeletalMeshModelingToolsEditorModeBinding(USkeletalMeshModelingToolsEditorMode* InEditorMode);

	virtual TSharedPtr<ISkeletalMeshNotifier> GetNotifier() override;
	virtual NameFunction GetNameFunction() override;
	virtual TArray<FName> GetSelectedBones() const override;


private:
	TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> EditorMode;	
	TSharedPtr<FSkeletalMeshModelingToolsEditorModeNotifier> Notifier;
};



UCLASS()
class USkeletalMeshModelingToolsEditorMode : 
	public UBaseLegacyWidgetEdMode,
	public ISkeletalMeshBackedDynamicMeshComponentProvider,
	public IGizmoEdModeInterface,
	public IMorphTargetManagerDataSource,
	public IHotkeyHintProvider
{
	GENERATED_BODY()
public:
	const static FEditorModeID Id;	

	USkeletalMeshModelingToolsEditorMode();
	explicit USkeletalMeshModelingToolsEditorMode(FVTableHelper& Helper);
	virtual ~USkeletalMeshModelingToolsEditorMode() override;

	// UEdMode overrides
	virtual void Initialize() override;
	virtual void Enter() override;
	virtual void Exit() override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual void CreateToolkit() override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual bool UsesPropertyWidgets() const override { return false; }

	virtual void Tick(FEditorViewportClient* InViewportClient, float InDeltaTime) override;
	virtual bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const override;
	virtual bool UsesToolkits() const override { return true; }
	virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;
	virtual void RegisterTool(TSharedPtr<FUICommandInfo> UICommand, FString ToolIdentifier, UInteractiveToolBuilder* Builder, EToolsContextScope ToolScope = EToolsContextScope::Default) override;

	/** Returns the FUICommandInfo that registered the given tool identifier (palette button command), or nullptr if not registered through RegisterTool. */
	TSharedPtr<FUICommandInfo> FindCommandForTool(const FString& ToolIdentifier) const;

	virtual bool MouseEnter(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 X, int32 Y) override;
	// virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	// virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool UsesTransformWidget() const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click) override;
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	virtual void PostUndo() override;
	virtual bool OnRequestClose() override;

	// ISkeletalMeshBackedDynamicMeshComponentProvider
	virtual USkeletalMeshBackedDynamicMeshComponent* GetComponent(UObject* SourceObject) override;


	// IGizmoEdModeInterface
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	
	// binding
	TSharedPtr<ISkeletalMeshEditorBinding> GetModeBinding();

	void SetEditorBinding(const TWeakPtr<ISkeletalMeshEditor>& InSkeletalMeshEditor);
	TSharedPtr<ISkeletalMeshEditorBinding> GetEditorBinding();
	TSharedPtr<ISkeletalMeshEditor> GetEditor() const;
	
	USkeletalMesh* GetSkeletalMesh() const;
	void HandleSkeletalMeshPreChange();
	void HandleSkeletalMeshChanged();
	
	bool CanSetEditingLOD();
	void SetEditingLOD(EMeshLODIdentifier EditingLOD);
	EMeshLODIdentifier GetEditingLOD();

	USkeletalMeshEditingCache* GetCurrentEditingCache() const;
	bool HasUnappliedChanges() const;
	void ApplyChanges();
	void DiscardChanges();
	
	enum class EToolAcceptAction: uint8
	{
		ApplyToAsset,
		ExitTool,
	};
	
	EToolAcceptAction GetToolAcceptAction() const;
	void SetToolAcceptAction(EToolAcceptAction InAction);

	void RequestApplyChangesToAssetOnToolEnd();
	
	void HideSkeletonForTool();	
	void ShowSkeletonForTool();
	void ToggleBoneManipulation(bool bEnable);
	USkeletonModifier* GetSkeletonReader();
	void SetSelectedBones(const TArray<FName>& InSelectedBones);
	TArray<FName> GetSelectedBones() const;
	
	// IMorphTargetManagerDataSource
	virtual TArray<FName> GetMorphTargets() override;
	virtual float         GetMorphTargetWeight(FName MorphTarget) override;
	virtual void          SetMorphTargetWeight(FName MorphTarget, float Weight) override;
	virtual bool          GetMorphTargetAutoFill(FName MorphTarget) override;
	virtual void          SetMorphTargetAutoFill(FName MorphTarget, bool bAutoFill, float PreviousOverrideWeight) override;
	virtual FName         GetEditingMorphTarget() const override;
	virtual void          SetEditingMorphTarget(FName MorphTarget) override;
	virtual FName         AddMorphTarget(FName InName) override;
	virtual TArray<FName> AddMorphTargetsIfMissing(const TArray<FName>& Names) override;
	virtual FName         RenameMorphTarget(FName OldName, FName NewName) override;
	virtual void          RemoveMorphTargets(const TArray<FName>& Names) override;
	virtual TArray<FName> DuplicateMorphTargets(const TArray<FName>& Names) override;
	virtual void          MirrorMorphTargets(const TArray<FName>& Names) override;
	virtual void          FlipMorphTargets(const TArray<FName>& Names) override;
	virtual FName         MergeMorphTargets(const TArray<FName>& Names) override;
	virtual void          ApplyCurrentWeightToMorphTarget(FName Name) override;
	virtual void          GenerateFlippedMorphTargets(const TArray<TPair<FName, FName>>& InPairs) override;
	virtual FSimpleMulticastDelegate& OnMorphTargetDataChanged() override { return OnMorphTargetDataChangedDelegate; }

	TSharedPtr<FTabManager> GetAssociatedTabManager();

	void BindToolSkeletonTree(ISkeletalMeshEditingInterface* InEditingInterface);
	void UnbindToolSkeletonTree();
	
	FSimpleMulticastDelegate& OnInitialized() { return OnInitializedDelegate; }

	UGeometrySelectionManager* GetSelectionManager() const;
	
protected:
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	void OnToolPostBuild(UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState);
	void OnToolEndedWithStatus(UInteractiveToolManager* Manager, UInteractiveTool* Tool, EToolShutdownType ShutdownType);
	
	void OnToolContextRender(IToolsContextRenderAPI* ToolsContextRenderAPI);
	void OnToolContextDrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* ToolsContextRenderAPI);

	void HandlePreSaveAsset();
private:
	EToolAcceptAction ToolAcceptAction = EToolAcceptAction::ApplyToAsset;
	bool bApplyChangesToAssetOnToolEnd = false;
	
	TSharedPtr<FSkeletalMeshModelingToolsEditorModeBinding> ModeBinding;

	TWeakPtr<ISkeletalMeshEditor> Editor;
	TWeakPtr<ISkeletalMeshEditorBinding> EditorBinding;
	TWeakObjectPtr<USkeletalMesh> SkeletalMesh;

	UPROPERTY(NonTransactional)
	TObjectPtr<USkeletalMeshEditingCache> CurrentEditingCache = nullptr;

	bool bExternalChangeDetected = false;
	void RecreateEditingCache(EMeshLODIdentifier InLOD);

	TUniquePtr<FSkeletalMeshNotifierBindScope> EditingCacheNotifierBindScope;

	// Represents the collection of bones currently selected in the viewport for whichever skeleton that is currently active
	// Helps maintain consistent bone selection state as different skeleton tree widget activates
	TArray<FName> SelectedBones;

	UPROPERTY()
	FName EditingMorphTarget = NAME_None;
	
	// Stylus support is currently disabled; this is left in for reference if/when it is brought back
	//TUniquePtr<FStylusStateTracker> StylusStateTracker;

	// we restore previous switch tool behavior when exiting this mode
	EToolManagerToolSwitchMode ToolSwitchModeToRestoreOnExit;

	static ISkeletalMeshEditingInterface* GetSkeletonInterface(UInteractiveTool* InTool);

	UDebugSkelMeshComponent* GetSkelMeshComponent() const;
	void ToggleSkeletalMeshBoneManipulation(bool bEnable);
	bool IsSkeletalMeshBoneManipulationEnabled();

	struct FDesiredSkeletonState
	{
		bool bVisible = true;
		bool bManipulation = true;
		friend bool operator==(const FDesiredSkeletonState&, const FDesiredSkeletonState&) = default;
	};

	FDesiredSkeletonState DesiredSkeletonState;
	
	// A dummy read-only modifier to host the current edited skeleton for RefSkeletonTree UI
	UPROPERTY(NonTransactional)
	TObjectPtr<USkeletonModifier> SkeletonReader = nullptr;

	struct FSkeletonTreeHostStateFactors
	{
		bool bEnableDynamicMeshSkeleton = false;
		bool bIsUsingToolSkeletonWidget = false;
		bool bActiveToolModifiesSkeleton = false;
		friend bool operator==(const FSkeletonTreeHostStateFactors&, const FSkeletonTreeHostStateFactors&) = default;
	};
	
	// Decides which skeleton tree widget to show to the user
	TValueWatcher<FSkeletonTreeHostStateFactors> SkeletonTreeHostStateUpdater;

	bool bIsUsingToolSkeletonWidget = false;
	bool bActiveToolModifiesSkeleton = false;
	
	TWeakPtr<FSkeletalMeshModelingToolsEditorModeToolkit> TypedToolkit = nullptr;
	
	bool bDeactivateOnPIEStartStateToRestore;
	bool bDeactivateOnSaveWorldStateToRestore;

	void RegisterExtensions();
	// Support extension tools having their own hotkey classes
	TMap<FString, FExtensionToolDescription> ExtensionToolToInfo;
	// Note: this will only work when the given tool is active, because we get the tool identifier
	//  out of the manager using GetActiveToolName
	bool TryGetExtensionToolCommandGetter(UInteractiveToolManager* InManager, const UInteractiveTool* InTool, 
		TFunction<const UE::IInteractiveToolCommandsInterface&()>& OutGetter) const;
	// Used to unbind extension tool commands
	TFunction<const UE::IInteractiveToolCommandsInterface& ()> ExtensionToolCommandsGetter;

	// Separate from the toolkit list so chord dispatch isn't short-circuited by a disabled mode
	// command sharing the chord (FUICommandList::ConditionalProcessCommandBindings short-circuits
	// on the first chord match regardless of CanExecute).
	TSharedPtr<FUICommandList> ActiveToolCommandList;
	TSharedPtr<FUICommandList> ActiveGeometrySelectionCommandList;

	bool bShowHotkeyHints = true;

	void DrawActiveToolHotkeyHints(FViewport* Viewport, FCanvas* Canvas) const;
	virtual void GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const override;

	FSimpleMulticastDelegate OnInitializedDelegate;
	FSimpleMulticastDelegate OnMorphTargetDataChangedDelegate;



	UPROPERTY(NonTransactional)
	TObjectPtr<UGeometrySelectionManager> SelectionManager;

	UPROPERTY(NonTransactional)
	TObjectPtr<USkeletalMeshGeometrySelectionTransformTweaker> SelectionTransformTweaker;
	
	// Handlers for USkeletalMeshEditingCache events. Wired up in RecreateEditingCache.
	void HandleEditingCacheComponentChanged();
	void HandleEditingCacheSkeletonChanged();
	void HandleEditingCachePreviewMeshDeformed();

	// GeometrySelectionStateWatcher writes DesiredSkeletonState.bManipulation, which
	// SkeletonStateUpdater reads — run them in order so the cascade settles in one pass.
	// Also bound to SelectionManager->OnSelectionModified so any manager-driven state change
	// (including undo/redo of selection mode) re-settles the derived state.
	void UpdateGeometrySelectionDerivedState();

	void SetSuspendGeometrySelection(bool bSuspend);
	void SetIsUsingToolSkeletonWidget(bool bUsing);
	void SetActiveToolModifiesSkeleton(bool bModifies);

	bool bSuspendGeometrySelection = false;

	void InitializeGeometrySelectionSystems();

	void OnBuildViewportInteractions(UViewportInteractionsBehaviorSource* Source);
	void PostBuildViewportInteractions(const UViewportInteractionsBehaviorSource* Source);

	// Used to keep the command objects alive
	UPROPERTY(NonTransactional)
	TArray<TObjectPtr<UInteractiveCommand>> ModelingModeCommands;

	TValueWatcher<bool> GeometrySelectionStateWatcher;

	bool IsGeometrySelectionActive() const;
	bool HasActiveGeometrySelection() const;
	void ToggleGeometrySelectionViewportInteractions(bool bEnable);
	
	// Geometry isolation (return false if no-op)
	bool IsolateSelection();
	bool HideSelection();
	bool ShowFullMesh();

	FString LastActivatedToolName;

	TMap<FString, TSharedPtr<FUICommandInfo>> ToolIdentifierToCommand;

	void ShowQuickAccessMenu();


	struct FSkeletonStateFactors
	{
		bool bIsGeometrySelectionActive = false;
		FDesiredSkeletonState DesiredSkeletonState;
		friend bool operator==(const FSkeletonStateFactors&, const FSkeletonStateFactors&) = default;
	};
	FSkeletonStateFactors GetSkeletonStateFactors();

	TValueWatcher<FSkeletonStateFactors> SkeletonStateUpdater;
	
	int32 ToolReactivateScopeDepth = 0;

	struct FToolReactivateScope
	{
		FToolReactivateScope(USkeletalMeshModelingToolsEditorMode* InMode);
		~FToolReactivateScope();
	private:
		USkeletalMeshModelingToolsEditorMode* Mode = nullptr;
		bool bNeedsReactivate = false;
	};
};
