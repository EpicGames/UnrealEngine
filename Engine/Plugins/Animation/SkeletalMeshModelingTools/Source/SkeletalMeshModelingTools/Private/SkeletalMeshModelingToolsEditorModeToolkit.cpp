// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshModelingToolsEditorModeToolkit.h"

#include "SkeletalMeshModelingToolsCommands.h"

#include "Curves/LinearColorRamp.h"
#include "Customizations/LinearColorRampCustomization.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Features/IModularFeatures.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "InteractiveTool.h"
#include "InteractiveToolsContext.h"
#include "ISkeletalMeshEditor.h"
#include "ModelingToolsEditorModeSettings.h"
#include "ModelingToolsEditorModeStyle.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Tools/UEdMode.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "SkeletalMeshModelingToolsStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "ModelingToolsManagerActions.h"
#include "ModelingToolTargetEditorOnlyUtil.h"
#include "PersonaTabs.h"
#include "SEnumCombo.h"
#include "SkeletalMeshEditingCache.h"
#include "SkeletalMeshModelingModeToolExtensions.h"
#include "SkeletalMeshModelingToolsEditorMode.h"
#include "SMorphTargetManager.h"
#include "BaseTools/MeshSurfacePointTool.h"
#include "SPrimaryButton.h"
#include "SkeletonModifier.h"
#include "SkeletalMesh/SReferenceSkeletonTree.h"
#include "Components/SKMBackedDynaMeshComponent.h"
#include "Editor/EditorWidgets/Public/SEnumCombo.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Docking/SDockTab.h"
#include "ScopedTransaction.h"
#include "Selection/GeometrySelectionManager.h"
#include "SkeletalMesh/SkeletalMeshModelingToolsEditorSettings.h"


#define LOCTEXT_NAMESPACE "SkeletalMeshModelingToolsEditorModeToolkit"

FSkeletalMeshModelingToolsEditorModeToolkit::~FSkeletalMeshModelingToolsEditorModeToolkit()
{
	if (UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode))
	{
		Context->OnToolNotificationMessage.RemoveAll(this);
		Context->OnToolWarningMessage.RemoveAll(this);
	}

	if (UModelingToolsModeCustomizationSettings* ModelingCustomizationSettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>())
	{
		ModelingCustomizationSettings->OnSettingChanged().RemoveAll(this);
	}
}

void FSkeletalMeshModelingToolsEditorModeToolkit::Init(
	const TSharedPtr<IToolkitHost>& InToolkitHost, 
	TWeakObjectPtr<UEdMode> InOwningMode)
{
	bUsesToolkitBuilder = true;

	FModeToolkit::Init(InToolkitHost, InOwningMode);

	SetupCommands();

	ModeWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ModeWarningArea->SetText(FText::GetEmpty());
	ModeWarningArea->SetVisibility(EVisibility::Collapsed);

	ModeHeaderArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12));
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ModeHeaderArea->SetJustification(ETextJustify::Center);
	
	ToolWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ToolWarningArea->SetText(FText::GetEmpty());


	RegisterPalettes();

	ClearNotification();
	ClearWarning();
	
	// create the toolkit widget
	{
		if (ModeDetailsView.IsValid())
		{
			constexpr FLinearColorRampCustomizationArgs Args({ .bWithAlphaChannel = false });

			ModeDetailsView->RegisterInstancedCustomPropertyTypeLayout(
				FLinearColorRamp::StaticStruct()->GetFName(),
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLinearColorRampCustomization::MakeInstance, Args));
		}

		ToolkitSections->ModeWarningArea = ModeWarningArea;
		ToolkitSections->DetailsView = ModeDetailsView;
		ToolkitSections->ToolWarningArea = ToolWarningArea;
		ToolkitSections->Footer = MakeFooterWidget();

		SAssignNew(ToolkitWidget, SBorder)
		.HAlign(HAlign_Fill)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			ToolkitBuilder->GenerateWidget()->AsShared()
		];	
	}
	
	ActiveToolName = FText::GetEmpty();
	ActiveToolMessage = FText::GetEmpty();

	if (UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode))
	{
		Context->OnToolNotificationMessage.AddSP(this, &FSkeletalMeshModelingToolsEditorModeToolkit::PostNotification);
		Context->OnToolWarningMessage.AddSP(this, &FSkeletalMeshModelingToolsEditorModeToolkit::PostWarning);
	}

	// add viewport overlay widget to accept / cancel tool
	MakeToolAcceptCancelWidget();

	
	EditorMode = CastChecked<USkeletalMeshModelingToolsEditorMode>(InOwningMode.Get());
	EditorMode->OnInitialized().AddSP(this, &FSkeletalMeshModelingToolsEditorModeToolkit::OnEditorModeInitialized);

	UModelingToolsModeCustomizationSettings* ModelingCustomizationSettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
	ModelingCustomizationSettings->OnSettingChanged().AddSP(SharedThis(this),  &FSkeletalMeshModelingToolsEditorModeToolkit::UpdateSelectionColors);


}

FName FSkeletalMeshModelingToolsEditorModeToolkit::GetToolkitFName() const
{
	return FName("SkeletalMeshModelingToolsEditorModeToolkit");
}

FText FSkeletalMeshModelingToolsEditorModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("DisplayName", "Skeletal Mesh Editing Tools");
}

void FSkeletalMeshModelingToolsEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	UpdateActiveToolProperties(Tool);

	Tool->OnPropertySetsModified.AddSP(this, &FSkeletalMeshModelingToolsEditorModeToolkit::UpdateActiveToolProperties, Tool);

	ModeHeaderArea->SetVisibility(EVisibility::Collapsed);

	// Prefer the launching command's label so the overlay title matches the palette button the user clicked,
	// rather than the tool's self-declared ToolDisplayName which can drift from the menu label.
	const FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager()->GetActiveToolName(EToolSide::Left);
	const TSharedPtr<FUICommandInfo> ActiveToolCommand = EditorMode.IsValid() ? EditorMode->FindCommandForTool(ActiveToolIdentifier) : nullptr;
	ActiveToolName = ActiveToolCommand.IsValid() ? ActiveToolCommand->GetLabel() : Tool->GetToolInfo().ToolDisplayName;

	FString ActiveToolIdentifierString = ActiveToolIdentifier;
	ActiveToolIdentifierString.InsertAt(0, ".");
	FName ActiveToolIconName = ISlateStyle::Join(FModelingToolsManagerCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifierString));
	ActiveToolIcon = FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(ActiveToolIconName);
	
	GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());

	AssetConfigPanel->SetExpanded_Animated(false);
}

void FSkeletalMeshModelingToolsEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}
	
	if (Tool)
	{
		Tool->OnPropertySetsModified.RemoveAll(this);
	}

	ModeHeaderArea->SetVisibility(EVisibility::Visible);
	ModeDetailsView->SetObject(nullptr);
	ActiveToolName = FText::GetEmpty();
	ClearNotification();
	ClearWarning();
	
	AssetConfigPanel->SetExpanded_Animated(true);
}

FText FSkeletalMeshModelingToolsEditorModeToolkit::GetActiveToolDisplayName() const
{
	return ActiveToolName;
}


FText FSkeletalMeshModelingToolsEditorModeToolkit::GetActiveToolMessage() const
{
	return ActiveToolMessage;
}

void FSkeletalMeshModelingToolsEditorModeToolkit::InvokeUI()
{
	// Skipping parent class call here since persona already spawns a tool box tab
	// invoking the UI here would create a second tool box tab
	// FModeToolkit::InvokeUI();
	
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		TSharedPtr<FTabManager> TabManager = ModeUILayerPtr->GetTabManager();
		

		SAssignNew(MorphTargetManagerWidget, SBorder)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"));
		
		// Morph Target
		{
			const FName& MorphTargetTabID = FPersonaTabs::MorphTargetsID;
			TSharedPtr<SDockTab> MorphTargetTab = TabManager->FindExistingLiveTab(MorphTargetTabID);
			if (!MorphTargetTab.IsValid())
			{
				MorphTargetTab = TabManager->TryInvokeTab(MorphTargetTabID);
			}
		
			{
				DefaultMorphTargetWidget = MorphTargetTab->GetContent();
			
				MorphTargetTab->SetContent(MorphTargetManagerWidget.ToSharedRef());
			}
		}	
	}

	
}

void FSkeletalMeshModelingToolsEditorModeToolkit::ShutdownUI()
{
	// Restore Skeleton UI
	EnableSkeletalMeshSkeletonCommands();
	ShowSkeletalMeshSkeletonTree();
	
	// Restore Morph Target	UI
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		TSharedPtr<FTabManager> TabManager = ModeUILayerPtr->GetTabManager();

		{
			const FName& MorphTargetTabID = FPersonaTabs::MorphTargetsID;
			TSharedPtr<SDockTab> MorphTargetTab = TabManager->FindExistingLiveTab(MorphTargetTabID);
			if (MorphTargetTab.IsValid())
			{
				// switch back to default widget if it was replaced before
				if (DefaultMorphTargetWidget)
				{
					MorphTargetTab->SetContent(DefaultMorphTargetWidget.ToSharedRef());
				}
			}
		}	
	}

	FModeToolkit::ShutdownUI();
}

TArray<FName> FSkeletalMeshModelingToolsEditorModeToolkit::GetSelectedBonesForDynamicMeshSkeleton()
{
	TArray<FName> SelectedBones;
	DynamicMeshSkeletonTree->GetSelectedBoneNames(SelectedBones);
	return SelectedBones;
}

void FSkeletalMeshModelingToolsEditorModeToolkit::ResetDynamicMeshBoneTransforms(bool bSelectedOnly)
{
	FText Title = bSelectedOnly ? LOCTEXT("ResetBoneTransforms", "Reset Bone Transforms") : LOCTEXT("ResetAllBonesTransforms", "Reset All Bone Transforms");
	
	FScopedTransaction Transaction(Title);
	
	if (EditorMode.IsValid() && EditorMode->GetCurrentEditingCache())
	{
		EditorMode->GetCurrentEditingCache()->ResetDynamicMeshBoneTransforms(bSelectedOnly);
	}
}


void FSkeletalMeshModelingToolsEditorModeToolkit::RegisterPalettes()
{
	const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();
	const TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();
	
	ToolkitSections = MakeShared<FToolkitSections>();
	FToolkitBuilderArgs ToolkitBuilderArgs(GetScriptableEditorMode()->GetModeInfo().ToolbarCustomizationName);
	ToolkitBuilderArgs.ToolkitCommandList = GetToolkitCommands();
	ToolkitBuilderArgs.ToolkitSections = ToolkitSections;
	ToolkitBuilderArgs.SelectedCategoryTitleVisibility = EVisibility::Collapsed;
	ToolkitBuilder = MakeShared<FToolkitBuilder>(ToolkitBuilderArgs);

	// Palette display order: Skin, Extensions, Deform, Model, Mesh, Bake, Misc.

	// Edit Skeleton lives at the top of the Skin palette rather than in its own group.
	const TArray<TSharedPtr<FUICommandInfo>> SkinCommands({
		Commands.BeginSkeletonEditingTool,
		Commands.BeginSkinWeightsBindingTool,
		Commands.BeginSkinWeightsPaintTool,
		Commands.BeginAttributeEditorTool,
		Commands.BeginMeshAttributePaintTool,
		Commands.BeginMeshVertexPaintTool,
		Commands.BeginPolyGroupsTool,
		Commands.BeginMeshGroupPaintTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadSkinTools.ToSharedRef(), SkinCommands ) ) );

	RegisterExtensionsPalettes();

	const TArray<TSharedPtr<FUICommandInfo>> DeformCommands({
		Commands.BeginSculptMeshTool,
		Commands.BeginRemeshSculptMeshTool,
		Commands.BeginSmoothMeshTool,
		Commands.BeginOffsetMeshTool,
		Commands.BeginMeshSpaceDeformerTool,
		Commands.BeginLatticeDeformerTool,
		Commands.BeginDisplaceMeshTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadDeformTools.ToSharedRef(), DeformCommands ) ) );

	const TArray<TSharedPtr<FUICommandInfo>> ModelingMeshCommands({
		Commands.BeginPolyEditTool,
		Commands.BeginTriEditTool,
		Commands.BeginPolyDeformTool,
		Commands.BeginHoleFillTool,
		Commands.BeginPolygonCutTool,
		});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadPolyTools.ToSharedRef(), ModelingMeshCommands ) ) );

	const TArray<TSharedPtr<FUICommandInfo>> ProcessMeshCommands({
		Commands.BeginSimplifyMeshTool,
		Commands.BeginRemeshMeshTool,
		Commands.BeginWeldEdgesTool,
		Commands.BeginRemoveOccludedTrianglesTool,
		Commands.BeginProjectToTargetTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadMeshOpsTools.ToSharedRef(), ProcessMeshCommands ) ) );

	const TArray<TSharedPtr<FUICommandInfo>> BakeCommands({
		Commands.BeginBakeMeshAttributeVertexTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( Commands.LoadBakingTools.ToSharedRef(), BakeCommands ) ) );

	const TArray<TSharedPtr<FUICommandInfo>> MiscCommands({
		FSkeletalMeshModelingToolsCommands::Get().BeginSkeletalMeshRunMeshProcessorBlueprintTool
	});
	ToolkitBuilder->AddPalette(
		MakeShareable( new FToolPalette( FSkeletalMeshModelingToolsCommands::Get().LoadMiscTools.ToSharedRef(), MiscCommands ) ) );

	ToolkitBuilder->SetActivePaletteOnLoad(Commands.LoadSkinTools.Get());
	ToolkitBuilder->UpdateWidget();
	
	// if selected palette changes, make sure we are showing the palette command buttons, which may be hidden by active Tool
	ActivePaletteChangedHandle = ToolkitBuilder->OnActivePaletteChanged.AddLambda([this]()
	{
		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);
	});
}

void FSkeletalMeshModelingToolsEditorModeToolkit::RegisterExtensionsPalettes() const
{
	const TArray<ISkeletalMeshModelingModeToolExtension*> Extensions = IModularFeatures::Get().GetModularFeatureImplementations<ISkeletalMeshModelingModeToolExtension>(
		ISkeletalMeshModelingModeToolExtension::GetModularFeatureName());

	if (Extensions.IsEmpty())
	{
		return;
	}

	for (ISkeletalMeshModelingModeToolExtension* Extension: Extensions)
	{
		const FText ExtensionName = Extension->GetExtensionName();
		const FText SectionName = Extension->GetToolSectionName();

		FModelingModeExtensionExtendedInfo ExtensionExtendedInfo;
		const bool bHasExtendedInfo = Extension->GetExtensionExtendedInfo(ExtensionExtendedInfo);

		TSharedPtr<FUICommandInfo> PaletteCommand;
		if (bHasExtendedInfo && ExtensionExtendedInfo.ExtensionCommand.IsValid())
		{
			PaletteCommand = ExtensionExtendedInfo.ExtensionCommand;
		}
		else
		{
			const FText UseTooltipText = (bHasExtendedInfo && ExtensionExtendedInfo.ToolPaletteButtonTooltip.IsEmpty() == false) ?
				ExtensionExtendedInfo.ToolPaletteButtonTooltip : SectionName;
			PaletteCommand = FModelingToolsManagerCommands::RegisterExtensionPaletteCommand(
				FName(ExtensionName.ToString()),
				SectionName, UseTooltipText, FSlateIcon());
		}

		TArray<TSharedPtr<FUICommandInfo>> PaletteItems;
		FExtensionToolQueryInfo ExtensionQueryInfo;
		ExtensionQueryInfo.bIsInfoQueryOnly = true;

		TArray<FExtensionToolDescription> ToolSet;
		Extension->GetExtensionTools(ExtensionQueryInfo, ToolSet);
		
		for (const FExtensionToolDescription& ToolInfo : ToolSet)
		{
			PaletteItems.Add(ToolInfo.ToolCommand);
		}

		ToolkitBuilder->AddPalette(
			MakeShareable(new FToolPalette(PaletteCommand.ToSharedRef(), PaletteItems)));
	}
}

void FSkeletalMeshModelingToolsEditorModeToolkit::MakeToolAcceptCancelWidget()
{
	SAssignNew(ViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FSkeletalMeshModelingToolsEditorModeToolkit::GetActiveToolDisplayName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([this]()
				{
					const UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext();
					return (Context->ActiveToolHasAccept() || Context->CanCompleteActiveTool()) ? EVisibility::Visible : EVisibility::Collapsed;
				})

				// Left side: "Apply to Asset" button (green, visible in ApplyOnToolExit mode)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				[
					SNew(SButton)
					.ButtonStyle(FSkeletalMeshModelingToolsStyle::Get(), "SkeletalMeshModelingTools.AcceptButton.ApplyToAsset")
					.Text(LOCTEXT("OverlayApplyToAsset", "Apply to Asset"))
					.ToolTipText(LOCTEXT("OverlayApplyToAssetToolTip", "Accept the current tool change and apply all pending changes to the skeletal mesh asset"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]()
					{
						EditorMode->RequestApplyChangesToAssetOnToolEnd();
						const UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext();
						Context->CanAcceptActiveTool()
							? GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept)
							: GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed);
						return FReply::Handled();
					})
					.IsEnabled_Lambda([this]()
					{
						const UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext();
						return Context->CanAcceptActiveTool() || Context->CanCompleteActiveTool();
					})
					.Visibility_Lambda([this]() { return EditorMode->GetToolAcceptAction() == USkeletalMeshModelingToolsEditorMode::EToolAcceptAction::ApplyToAsset ? EVisibility::Visible : EVisibility::Collapsed; })
				]

				// Left side: "Exit Tool" button (blue, visible in ApplyManually mode)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				[
					SNew(SButton)
					.ButtonStyle(FSkeletalMeshModelingToolsStyle::Get(), "SkeletalMeshModelingTools.AcceptButton.ExitTool")
					.Text(LOCTEXT("OverlayExitTool", "Exit Tool"))
					.ToolTipText(LOCTEXT("OverlayExitToolToolTip", "Accept changes and exit the tool without applying to the asset"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]()
					{
						const UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext();
						Context->CanAcceptActiveTool()
							? GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept)
							: GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed);
						return FReply::Handled();
					})
					.IsEnabled_Lambda([this]()
					{
						const UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext();
						return Context->CanAcceptActiveTool() || Context->CanCompleteActiveTool();
					})
					.Visibility_Lambda([this]() { return EditorMode->GetToolAcceptAction() == USkeletalMeshModelingToolsEditorMode::EToolAcceptAction::ExitTool ? EVisibility::Visible : EVisibility::Collapsed; })
				]

				// Right side: dropdown arrow for alternative action
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Fill)
				[
					SNew(SComboButton)
					.HasDownArrow(true)
					.ContentPadding(0.f)
					.ComboButtonStyle(FSkeletalMeshModelingToolsStyle::Get(), "SkeletalMeshModelingTools.AcceptButton.ComboButton")
					.MenuPlacement(MenuPlacement_ComboBoxRight)
					.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
					{
						FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

						const UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext();
						const bool bCanAccept = Context->CanAcceptActiveTool();
						const FText AcceptOrComplete = bCanAccept
							? LOCTEXT("AcceptVerb", "Accept")
							: LOCTEXT("CompleteVerb", "Complete");

						if (EditorMode->GetToolAcceptAction() == USkeletalMeshModelingToolsEditorMode::EToolAcceptAction::ApplyToAsset)
						{
							MenuBuilder.AddMenuEntry(
								LOCTEXT("SwitchToExitTool", "Apply Later"),
								LOCTEXT("SwitchToExitToolTip", "Exit the tool without applying changes to the asset. Changes can be applied later."),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateLambda([this]()
								{
									EditorMode->SetToolAcceptAction(USkeletalMeshModelingToolsEditorMode::EToolAcceptAction::ExitTool);
									const UEditorInteractiveToolsContext* Ctx = GetScriptableEditorMode()->GetInteractiveToolsContext();
									Ctx->CanAcceptActiveTool()
										? GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept)
										: GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed);
								}))
							);
						}
						else
						{
							const bool bIsAcceptTool = GetScriptableEditorMode()->GetInteractiveToolsContext()->CanAcceptActiveTool();
							int32 ChangeCount = EditorMode->GetCurrentEditingCache()->GetEditingMeshComponent()->GetChangeCount();
							ChangeCount += (bIsAcceptTool ? 1 : 0);
								
							MenuBuilder.AddMenuEntry(
								FText::Format(LOCTEXT("SwitchToApplyToAsset", "Apply {0} {0}|plural(one=Change,other=Changes) to Asset"), FText::AsNumber(ChangeCount)),
								LOCTEXT("SwitchToApplyToAssetTip", "Apply the current tool change and all pending changes to the skeletal mesh asset."),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateLambda([this]()
								{
									EditorMode->SetToolAcceptAction(USkeletalMeshModelingToolsEditorMode::EToolAcceptAction::ApplyToAsset);
									EditorMode->RequestApplyChangesToAssetOnToolEnd();
									const UEditorInteractiveToolsContext* Ctx = GetScriptableEditorMode()->GetInteractiveToolsContext();
									Ctx->CanAcceptActiveTool()
										? GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept)
										: GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed);
								}))
							);
						}

						return MenuBuilder.MakeWidget();
					})
					.IsEnabled_Lambda([this]()
					{
						const UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext();
						return Context->CanAcceptActiveTool() || Context->CanCompleteActiveTool();
					})
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ButtonStyle(FSkeletalMeshModelingToolsStyle::Get(), "SkeletalMeshModelingTools.CancelButton")
				.Text(LOCTEXT("OverlayCancel", "Cancel"))
				.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel); return FReply::Handled(); })
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCancelActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]
		]	
	];
}

void FSkeletalMeshModelingToolsEditorModeToolkit::SetupCommands()
{
	const FSkeletalMeshModelingToolsCommands& Commands = FSkeletalMeshModelingToolsCommands::Get();
	
	GetToolkitCommands()->MapAction(
		Commands.ToggleShowAllLocalRotationAxes,
		FUIAction(
			FExecuteAction::CreateLambda([]()
				{
					USkeletalMeshModelingToolsEditorSettings* const Settings = GetMutableDefault<USkeletalMeshModelingToolsEditorSettings>();
					Settings->SetShowAllLocalRotationAxes(!Settings->GetShowAllLocalRotationAxes());
					Settings->SaveConfig();
				}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([]()
				{
					const USkeletalMeshModelingToolsEditorSettings* const Settings = GetDefault<USkeletalMeshModelingToolsEditorSettings>();
					return Settings->GetShowAllLocalRotationAxes();
				})
		)
	);
}

void FSkeletalMeshModelingToolsEditorModeToolkit::PostNotification(const FText& InMessage)
{
	ClearNotification();
	
	ActiveToolMessage = InMessage;

	if (ModeUILayer.IsValid())
	{
		const FName StatusBarName = ModeUILayer.Pin()->GetStatusBarName();
		ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(StatusBarName, ActiveToolMessage);
	}
}


void FSkeletalMeshModelingToolsEditorModeToolkit::ClearNotification()
{
	ActiveToolMessage = FText::GetEmpty();

	if (ModeUILayer.IsValid())
	{
		const FName StatusBarName = ModeUILayer.Pin()->GetStatusBarName();
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(StatusBarName, ActiveToolMessageHandle);
	}
	ActiveToolMessageHandle.Reset();
}


void FSkeletalMeshModelingToolsEditorModeToolkit::PostWarning(const FText& Message)
{
	ToolWarningArea->SetText(Message);
	ToolWarningArea->SetVisibility(EVisibility::Visible);
}


void FSkeletalMeshModelingToolsEditorModeToolkit::ClearWarning()
{
	ToolWarningArea->SetText(FText());
	ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}

void FSkeletalMeshModelingToolsEditorModeToolkit::OnEditorModeInitialized()
{
	USkeletalMesh* SkeletalMesh = EditorMode->GetSkeletalMesh();

	constexpr bool bSkipAutoGenerated= true;
	AssetAvailableLODs = UE::ToolTarget::GetAvailableLODs(SkeletalMesh, bSkipAutoGenerated);
	ensure(!AssetAvailableLODs.IsEmpty());
	if (AssetAvailableLODs.IsEmpty())
	{
		AssetAvailableLODs.Add(EMeshLODIdentifier::LOD0);
	}
	
	for (EMeshLODIdentifier LOD : AssetAvailableLODs)
	{
		int32 LODIndex = static_cast<int32>(LOD);
		FString LODName = TEXT("LOD") + FString::FromInt(LODIndex);
		AssetLODModes.Add(MakeShared<FString>(LODName));
	}

	AssetLODMode->SetSelectedItem(AssetLODModes[0]);

	// Create a Skeleton Tree for dynamic mesh skeleton
	SAssignNew(DynamicMeshSkeletonWidget, SBorder)
	.Padding(4.f)
	.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	[
		SAssignNew(DynamicMeshSkeletonTree, SReferenceSkeletonTree)
			.Modifier(EditorMode->GetSkeletonReader())
	];


	SAssignNew(ToolSkeletonTreeHost, SBorder)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"));
	
	// Save the default Skeleton Tree
	if (ensure(ModeUILayer.IsValid()))
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		TSharedPtr<FTabManager> TabManager = ModeUILayerPtr->GetTabManager();


		const FName& SkeletonTreeId = FPersonaTabs::SkeletonTreeViewID;
		TSharedPtr<SDockTab> SkeletonTab = TabManager->FindExistingLiveTab(SkeletonTreeId);
		if (!SkeletonTab.IsValid())
		{
			SkeletonTab = TabManager->TryInvokeTab(SkeletonTreeId);
		}
		DefaultSkeletonWidget = SkeletonTab->GetContent();
	}

	// Save the default Skeleton bone manipulation commands
	const TSharedRef<FUICommandList>& CommandList = EditorMode->GetEditor()->GetToolkitCommands();
	DefaultResetBoneTransformsAction = *(CommandList->GetActionForCommand(EditorMode->GetEditor()->GetResetBoneTransformsCommand()));
	DefaultResetAllBoneTransformsAction = *(CommandList->GetActionForCommand(EditorMode->GetEditor()->GetResetAllBonesTransformsCommand()));

	

	SAssignNew(MorphTargetManager, SMorphTargetManager)
			.DataSource(EditorMode.Get());
	MorphTargetManagerWidget->SetContent(MorphTargetManager.ToSharedRef());

}

void FSkeletalMeshModelingToolsEditorModeToolkit::UpdateSelectionColors(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (EditorMode->GetSelectionManager())
	{
		const UModelingToolsModeCustomizationSettings* UISettings = GetMutableDefault<UModelingToolsModeCustomizationSettings>();
		EditorMode->GetSelectionManager()->SetSelectionColors(UISettings->UnselectedColor, UISettings->HoverOverSelectedColor, UISettings->HoverOverUnselectedColor, UISettings->GeometrySelectedColor, UISettings->GeometrySoftSelectedColor);
	}
}

void FSkeletalMeshModelingToolsEditorModeToolkit::ShowDynamicMeshSkeletonTree()
{
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		TSharedPtr<FTabManager> TabManager = ModeUILayerPtr->GetTabManager();

		const FName& SkeletonTreeId = FPersonaTabs::SkeletonTreeViewID;
		TSharedPtr<SDockTab> SkeletonTab = TabManager->FindExistingLiveTab(SkeletonTreeId);
		if (!SkeletonTab.IsValid())
		{
			SkeletonTab = TabManager->TryInvokeTab(SkeletonTreeId);
		}

		SkeletonTab->SetContent(DynamicMeshSkeletonWidget.ToSharedRef());

		// Make sure viewport selection is routed to the dynamic mesh skeleton tree
		DynamicMeshSkeletonTreeNotifierBindScope.Reset(
			new FSkeletalMeshNotifierBindScope(
				EditorMode->GetModeBinding()->GetNotifier(),
				DynamicMeshSkeletonTree->GetNotifier()));

		TArray<FName> SelectedBones = EditorMode->GetSelectedBones();
		DynamicMeshSkeletonTree->GetNotifier()->HandleNotification({}, ESkeletalMeshNotifyType::HierarchyChanged);
		DynamicMeshSkeletonTree->GetNotifier()->HandleNotification(SelectedBones, ESkeletalMeshNotifyType::BonesSelected);
		
		SkeletonTreeTabState = ESkeletonTabState::DisplayDynamicMeshSkeletonTree;
	}
}

void FSkeletalMeshModelingToolsEditorModeToolkit::ShowSkeletalMeshSkeletonTree()
{
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		TSharedPtr<FTabManager> TabManager = ModeUILayerPtr->GetTabManager();

		const FName& SkeletonTreeViewID = FPersonaTabs::SkeletonTreeViewID;
		TSharedPtr<SDockTab> SkeletonTab = TabManager->FindExistingLiveTab(SkeletonTreeViewID);
		if (!SkeletonTab.IsValid())
		{
			SkeletonTab = TabManager->TryInvokeTab(SkeletonTreeViewID);
		}

		SkeletonTab->SetContent(DefaultSkeletonWidget.ToSharedRef());

		DynamicMeshSkeletonTreeNotifierBindScope.Reset();
		DynamicMeshSkeletonTree->GetNotifier()->HandleNotification({}, ESkeletalMeshNotifyType::BonesSelected);
		
		SkeletonTreeTabState = ESkeletonTabState::DisplayDefaultSkeletalMeshSkeletonTree;
	}	
}

void FSkeletalMeshModelingToolsEditorModeToolkit::ShowToolSkeletonTree()
{
	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		TSharedPtr<FTabManager> TabManager = ModeUILayerPtr->GetTabManager();

		const FName& SkeletonTreeViewID = FPersonaTabs::SkeletonTreeViewID;
		TSharedPtr<SDockTab> SkeletonTab = TabManager->FindExistingLiveTab(SkeletonTreeViewID);
		if (!SkeletonTab.IsValid())
		{
			SkeletonTab = TabManager->TryInvokeTab(SkeletonTreeViewID);
		}

		SkeletonTab->SetContent(ToolSkeletonTreeHost.ToSharedRef());
		
		SkeletonTreeTabState = ESkeletonTabState::DisplayToolSkeletonTree;
	}		
}

void FSkeletalMeshModelingToolsEditorModeToolkit::RefreshDynamicMeshSkeletonTreeIfNeeded()
{
	if (SkeletonTreeTabState == ESkeletonTabState::DisplayDynamicMeshSkeletonTree)
	{
		DynamicMeshSkeletonTree->GetNotifier()->HandleNotification({}, ESkeletalMeshNotifyType::HierarchyChanged);
		DynamicMeshSkeletonTree->GetNotifier()->HandleNotification(EditorMode->GetSelectedBones(), ESkeletalMeshNotifyType::BonesSelected);
	}
}

void FSkeletalMeshModelingToolsEditorModeToolkit::EnableDynamicMeshSkeletonCommands()
{
	const TSharedPtr<ISkeletalMeshEditor> Editor = EditorMode.IsValid() ? EditorMode->GetEditor() : nullptr;
	if (!Editor.IsValid())
	{
		return;
	}

	const TSharedRef<FUICommandList>& CommandList = Editor->GetToolkitCommands();

	// Reset transforms
	static constexpr bool bSelectedOnly = true;
	CommandList->MapAction(Editor->GetResetBoneTransformsCommand(),
		FExecuteAction::CreateSP(this, &FSkeletalMeshModelingToolsEditorModeToolkit::ResetDynamicMeshBoneTransforms, bSelectedOnly));

	CommandList->MapAction(Editor->GetResetAllBonesTransformsCommand(),
		FExecuteAction::CreateSP(this, &FSkeletalMeshModelingToolsEditorModeToolkit::ResetDynamicMeshBoneTransforms, !bSelectedOnly));
}

void FSkeletalMeshModelingToolsEditorModeToolkit::EnableSkeletalMeshSkeletonCommands()
{
	const TSharedPtr<ISkeletalMeshEditor> Editor = EditorMode.IsValid() ? EditorMode->GetEditor() : nullptr;
	if (!Editor.IsValid())
	{
		return;
	}

	// Rebind commands if it was replaced before
	const TSharedRef<FUICommandList>& CommandList = Editor->GetToolkitCommands();

	// Reset transforms
	CommandList->MapAction(Editor->GetResetBoneTransformsCommand(), DefaultResetBoneTransformsAction);
	CommandList->MapAction(Editor->GetResetAllBonesTransformsCommand(), DefaultResetAllBoneTransformsAction);	
}

void FSkeletalMeshModelingToolsEditorModeToolkit::DisableEditingCacheSkeletonCommands()
{
	const TSharedPtr<ISkeletalMeshEditor> Editor = EditorMode.IsValid() ? EditorMode->GetEditor() : nullptr;
	if (!Editor.IsValid())
	{
		return;
	}

	// Rebind commands if it was replaced before
	const TSharedRef<FUICommandList>& CommandList = Editor->GetToolkitCommands();

	// Reset transforms
	CommandList->UnmapAction(Editor->GetResetBoneTransformsCommand());
	CommandList->UnmapAction(Editor->GetResetAllBonesTransformsCommand());	
}

bool FSkeletalMeshModelingToolsEditorModeToolkit::BindToolSkeletonTree(ISkeletalMeshEditingInterface* InEditingInterface)
{
	TSharedPtr<SWidget> ToolSkeletonTree = InEditingInterface->GetCustomSkeletonTreeWidget(EditorMode->GetModeBinding());

	if (!ToolSkeletonTree.IsValid())
	{
		return false;
	}

	ToolSkeletonTreeHost->SetContent(ToolSkeletonTree.ToSharedRef());
	
	return true;
}

void FSkeletalMeshModelingToolsEditorModeToolkit::UnbindToolSkeletonTree()
{
	ToolSkeletonTreeHost->SetContent(SNullWidget::NullWidget);
}

TSharedPtr<SWidget> FSkeletalMeshModelingToolsEditorModeToolkit::MakeFooterWidget()
{
	// LOD picker widget
	{
		AssetLODMode = SNew(STextComboBox)
			.OptionsSource(&AssetLODModes)
			.IsEnabled(this, &FSkeletalMeshModelingToolsEditorModeToolkit::CanChangeAssetEditingSettings)
			.OnSelectionChanged_Lambda([this](TSharedPtr<FString> String, ESelectInfo::Type)
			{
				int32 Index = AssetLODModes.IndexOfByKey(String);
				check(AssetAvailableLODs.Num() == AssetLODModes.Num());
				EMeshLODIdentifier NewSelectedLOD = AssetAvailableLODs[Index];

				if (EditorMode.IsValid())
				{
					EditorMode->SetEditingLOD(NewSelectedLOD);
				}
			});

		AssetLODModeLabel = SNew(STextBlock)
		.Text(LOCTEXT("ActiveLODLabel", "Editing LOD"))
		.ToolTipText(LOCTEXT("ActiveLODLabelToolTip", "Select the LOD to be used when editing an existing mesh."));
	}


	

	
	const TSharedPtr<SVerticalBox> Content = SNew(SVerticalBox)
	+ SVerticalBox::Slot().HAlign(HAlign_Fill)
	.Padding(0, 8, 0, 4)
		[
		
			SNew(SVerticalBox)	
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().Padding(0).HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
					[ AssetLODModeLabel->AsShared() ]
				+ SHorizontalBox::Slot().Padding(0).HAlign(HAlign_Fill).Padding(0).FillWidth(4.f)
					[ AssetLODMode->AsShared() ]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.HeightOverride(50.f)
					.HAlign(HAlign_Fill)
					.Padding(0, 10, 5,10)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
						.ToolTipText(LOCTEXT("DiscardButtonToolTip", "Discard Pending Changes"))
						.ForegroundColor(FLinearColor::White)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.IsEnabled(this, &FSkeletalMeshModelingToolsEditorModeToolkit::IsDiscardButtonEnabled)
						.OnClicked(this, &FSkeletalMeshModelingToolsEditorModeToolkit::OnDiscardButtonPressed)
						[
							SNew( SImage )
							.Image(FAppStyle::GetBrush("Icons.Delete"))
							.ColorAndOpacity( FSlateColor::UseForeground() )
						]		
					]	
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SBox)
					.HeightOverride(50.f)
					.HAlign(HAlign_Fill)
					.Padding(5, 10, 0, 10)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), FName("FlatButton.Success"))
						.ForegroundColor(FLinearColor::White)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.IsEnabled(this, &FSkeletalMeshModelingToolsEditorModeToolkit::IsApplyButtonEnabled)
						.OnClicked(this, &FSkeletalMeshModelingToolsEditorModeToolkit::OnApplyButtonPressed)
						[
							SNew(STextBlock)
							.Text(this, &FSkeletalMeshModelingToolsEditorModeToolkit::GetApplyButtonText)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]		
					]	
				]	
			
			]
		];

		AssetConfigPanel = SNew(SExpandableArea)
		.HeaderPadding(FMargin(0.f))
		.Padding(FMargin(8.f))
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
		.AreaTitleFont(FAppStyle::Get().GetFontStyle("EditorModesPanel.CategoryFontStyle"))
		.BodyContent()
		[
			Content->AsShared()
		]
		.HeaderContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().HAlign(HAlign_Left).VAlign(VAlign_Center).FillWidth(2.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SkeletalMeshModelingToolFooterHeader", "Asset Editing Settings"))
			]	
		];
	
	return AssetConfigPanel;
}

bool FSkeletalMeshModelingToolsEditorModeToolkit::IsApplyButtonEnabled() const
{
	return CanChangeAssetEditingSettings() && EditorMode->HasUnappliedChanges();
}

FReply FSkeletalMeshModelingToolsEditorModeToolkit::OnApplyButtonPressed()
{
	FScopedTransaction Transaction(LOCTEXT("ApplyChangesToAsset","Apply Changes To Skeletal Mesh"));
	EditorMode->ApplyChanges();
	
	return FReply::Handled();
}

FText FSkeletalMeshModelingToolsEditorModeToolkit::GetApplyButtonText() const
{
	int32 ChangeCount = EditorMode->GetCurrentEditingCache() ? EditorMode->GetCurrentEditingCache()->GetEditingMeshComponent()->GetChangeCount() : 0;
	if (ChangeCount == 0)
	{
		return FText::Format(LOCTEXT("ApplyButtonZeroChangeText", "No Pending Changes"), FText::AsNumber(ChangeCount));
	}
	
	return FText::Format(LOCTEXT("ApplyButtonChangeText", "Apply {0} {0}|plural(one=Change, other=Changes)"), FText::AsNumber(ChangeCount));
}

bool FSkeletalMeshModelingToolsEditorModeToolkit::IsDiscardButtonEnabled() const
{
	return IsApplyButtonEnabled();
}

FReply FSkeletalMeshModelingToolsEditorModeToolkit::OnDiscardButtonPressed()
{
	FScopedTransaction Transaction(LOCTEXT("DiscardChanges","Discard Changes"));
	EditorMode->DiscardChanges();
	return FReply::Handled();
}

bool FSkeletalMeshModelingToolsEditorModeToolkit::CanChangeAssetEditingSettings() const
{
	return !GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->HasAnyActiveTool();
}

void FSkeletalMeshModelingToolsEditorModeToolkit::UpdateActiveToolProperties(UInteractiveTool* Tool)
{
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	if (CurTool == nullptr)
	{
		return;
	}
		
	ModeDetailsView->SetObjects(CurTool->GetToolProperties(true));
}

#undef LOCTEXT_NAMESPACE
