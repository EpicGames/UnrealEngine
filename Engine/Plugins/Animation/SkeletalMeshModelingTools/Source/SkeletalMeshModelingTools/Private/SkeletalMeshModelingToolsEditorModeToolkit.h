// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SkeletalMeshEditorUtils.h"
#include "SkeletalMeshModelingToolsEditorMode.h"
#include "StatusBarSubsystem.h"
#include "Toolkits/BaseToolkit.h"


class SMorphTargetManager;
class USkeletalMeshModelingToolsEditorMode;
enum class EMeshLODIdentifier;
class IToolkitHost;
class STextBlock;


class FSkeletalMeshModelingToolsEditorModeToolkit : public FModeToolkit
{
public:

	virtual ~FSkeletalMeshModelingToolsEditorModeToolkit() override;


	// IToolkit overrides
	virtual void Init(const TSharedPtr<IToolkitHost>& InToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual TSharedPtr<SWidget> GetInlineContent() const override { return ToolkitWidget; }

	// FModeToolkit overrides
	virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	virtual FText GetActiveToolDisplayName() const override;
	virtual FText GetActiveToolMessage() const override;
	virtual void InvokeUI() override;
	virtual void ShutdownUI() override;

	TArray<FName> GetSelectedBonesForDynamicMeshSkeleton();
	void ResetDynamicMeshBoneTransforms(bool bSelectedOnly);
	void ShowDynamicMeshSkeletonTree();
	void ShowSkeletalMeshSkeletonTree();
	void ShowToolSkeletonTree();
	
	void EnableDynamicMeshSkeletonCommands();
	void EnableSkeletalMeshSkeletonCommands();
	// When tool has its own skeleton, the editing cache skeleton is hidden and shouldn't receive commands like reset bone transforms
	void DisableEditingCacheSkeletonCommands();

	bool BindToolSkeletonTree(ISkeletalMeshEditingInterface* InTool);
	void UnbindToolSkeletonTree();
	
	void RefreshDynamicMeshSkeletonTreeIfNeeded();
private:
	void SetupCommands();

	void PostNotification(const FText& Message);
	void ClearNotification();

	void PostWarning(const FText& Message);
	void ClearWarning();


	void OnEditorModeInitialized();
	void UpdateSelectionColors(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent) const;

	TSharedPtr<SWidget> MakeFooterWidget();
	bool IsApplyButtonEnabled() const;
	FReply OnApplyButtonPressed();
	FText GetApplyButtonText() const;
	bool IsDiscardButtonEnabled() const;
	FReply OnDiscardButtonPressed();
	bool CanChangeAssetEditingSettings() const;
	
	void UpdateActiveToolProperties(UInteractiveTool* Tool);

	void RegisterPalettes();
	void RegisterExtensionsPalettes() const;
	FDelegateHandle ActivePaletteChangedHandle;

	void MakeToolAcceptCancelWidget();

	FText ActiveToolName;
	FText ActiveToolMessage;
	FStatusBarMessageHandle ActiveToolMessageHandle;

	TSharedPtr<SWidget> ToolkitWidget;
	TSharedPtr<IDetailsView> DetailsView;

	TSharedPtr<SWidget> ViewportOverlayWidget;
	const FSlateBrush* ActiveToolIcon = nullptr;
	
	TSharedPtr<STextBlock> ModeWarningArea;
	TSharedPtr<STextBlock> ModeHeaderArea;
	TSharedPtr<STextBlock> ToolWarningArea;

	TSharedPtr<SExpandableArea> AssetConfigPanel;
	
	TArray<EMeshLODIdentifier> AssetAvailableLODs;
	TArray<TSharedPtr<FString>> AssetLODModes;
	TSharedPtr<STextBlock> AssetLODModeLabel;
	TSharedPtr<STextComboBox> AssetLODMode;

	
	TSharedPtr<SWidget> DefaultSkeletonWidget;
	TSharedPtr<SBorder> DynamicMeshSkeletonWidget;
	TSharedPtr<SReferenceSkeletonTree> DynamicMeshSkeletonTree;

	TSharedPtr<SBorder> ToolSkeletonTreeHost;
	

	TSharedPtr<SWidget> DefaultMorphTargetWidget; 
	TSharedPtr<SBorder> MorphTargetManagerWidget; 
	TSharedPtr<SMorphTargetManager> MorphTargetManager; 

	TUniquePtr<FSkeletalMeshNotifierBindScope> DynamicMeshSkeletonTreeNotifierBindScope;

	TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> EditorMode;

	FUIAction DefaultResetBoneTransformsAction;
	FUIAction DefaultResetAllBoneTransformsAction;
	
	enum ESkeletonTabState
	{
		DisplayDefaultSkeletalMeshSkeletonTree,
		DisplayDynamicMeshSkeletonTree,
		DisplayToolSkeletonTree,
	};
	
	ESkeletonTabState SkeletonTreeTabState = ESkeletonTabState::DisplayDefaultSkeletalMeshSkeletonTree;
};
