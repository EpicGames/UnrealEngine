// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshModelingToolsMeshConverter.h"

#include "AssetToolsModule.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "PropertyEditorModule.h"
#include "StaticToSkeletalMeshConverter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Dialog/SCustomDialog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshModelingToolsMeshConverter)

DEFINE_LOG_CATEGORY_STATIC(LogSkeletalMeshModelingToolsMeshConverter, Log, All)

#define LOCTEXT_NAMESPACE "SkeletalMeshModelingToolsMeshConverter"

static void ShowEditorMessage(ELogVerbosity::Type InMessageType, const FText& InMessage, const FText* InLogMessage)
{
	FNotificationInfo Notification(InMessage);
	Notification.bUseSuccessFailIcons = true;
	Notification.FadeOutDuration = 5.0f;

	SNotificationItem::ECompletionState State = SNotificationItem::CS_Success;

	switch(InMessageType)
	{
	case ELogVerbosity::Warning:
		UE_LOGF(LogSkeletalMeshModelingToolsMeshConverter, Warning, "%ls", InLogMessage ? *InLogMessage->ToString() : *InMessage.ToString());
		break;
	case ELogVerbosity::Error:
		State = SNotificationItem::CS_Fail;
		UE_LOGF(LogSkeletalMeshModelingToolsMeshConverter, Error, "%ls", InLogMessage ? *InLogMessage->ToString() : *InMessage.ToString());
		break;
	default:
		checkNoEntry();
	}
	
	FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(State);
}



USkeletonFromStaticMeshFactory::USkeletonFromStaticMeshFactory(
	const FObjectInitializer& InObjectInitializer
	) :
	Super(InObjectInitializer)
{
	SupportedClass = USkeleton::StaticClass();
	bCreateNew = true;
	bEditAfterNew = false;
}


UObject* USkeletonFromStaticMeshFactory::FactoryCreateNew(
	UClass* InClass,
	UObject* InParent, 
	FName InName,
	EObjectFlags InFlags,
	UObject* InContext,
	FFeedbackContext* InWarn
	)
{
	USkeleton* Skeleton = NewObject<USkeleton>(InParent, InName, InFlags);

	FVector WantedRootPosition;
	switch(PositionReference)
	{
	case ERootBonePositionReference::Absolute:
		WantedRootPosition = RootPosition;
		break;
		
	case ERootBonePositionReference::Relative:
		{
			const FBox Bounds = StaticMesh->GetBoundingBox();
			WantedRootPosition = Bounds.Min + (Bounds.Max - Bounds.Min) * RootPosition;		
		}
		break;
		
	default:
		checkNoEntry()
		break;
	}

	const TCHAR* RootBoneName = TEXT("Root"); 
	FTransform RootTransform(FTransform::Identity);
	RootTransform.SetTranslation(WantedRootPosition);

	FReferenceSkeletonModifier Modifier(Skeleton);
	Modifier.Add(FMeshBoneInfo(RootBoneName, RootBoneName, INDEX_NONE), RootTransform);

	return Skeleton;
}


USkeletalMeshFromStaticMeshFactory::USkeletalMeshFromStaticMeshFactory(
	const FObjectInitializer& InObjectInitializer
	) :
	Super(InObjectInitializer)
{
	SupportedClass = USkeletalMesh::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


UObject* USkeletalMeshFromStaticMeshFactory::FactoryCreateNew(
	UClass* InClass,
	UObject* InParent,
	FName InName,
	EObjectFlags InFlags,
	UObject* InContext,
	FFeedbackContext* InWarn
	)
{
	USkeletalMesh* SkeletalMesh = NewObject<USkeletalMesh>(InParent, InName, InFlags);
	
	if (!FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromStaticMesh(SkeletalMesh, StaticMesh, ReferenceSkeleton, BindBoneName))
	{
		return nullptr;
	}
	
	// Update the skeletal mesh and the skeleton so that their ref skeletons are in sync and the skeleton's preview mesh
	// is the one we just created.
	SkeletalMesh->SetSkeleton(Skeleton);
	Skeleton->MergeAllBonesToBoneTree(SkeletalMesh);
	if (!Skeleton->GetPreviewMesh())
	{
		Skeleton->SetPreviewMesh(SkeletalMesh);
	}
	
	return SkeletalMesh;
}


USkeleton* UStaticMeshToSkeletalMeshConvertOptions::GetSkeleton(
	bool& bInvalidSkeletonIsError,
	const IPropertyHandle* PropertyHandle)
{
	// bInvalidSkeletonIsError = true;

	switch (SkeletonImportOption)
	{
	default:
	case EReferenceSkeletonImportOption::CreateNew:
		return nullptr;
	case EReferenceSkeletonImportOption::UseExistingSkeleton:
		return Cast<USkeleton>(Skeleton.ResolveObject());
	case EReferenceSkeletonImportOption::UseExistingSkeletalMesh:
		{
			USkeletalMesh* SkeletalMeshPtr = Cast<USkeletalMesh>(SkeletalMesh.ResolveObject());
			return SkeletalMeshPtr ? SkeletalMeshPtr->GetSkeleton() : nullptr;
		}
	}
}


void UStaticMeshToSkeletalMeshConvertOptions::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropertyName = PropertyChangedEvent.Property->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UStaticMeshToSkeletalMeshConvertOptions, SkeletonImportOption) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UStaticMeshToSkeletalMeshConvertOptions, Skeleton) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(UStaticMeshToSkeletalMeshConvertOptions, SkeletalMesh))
		{
			// If the pointed-at-bone doesn't exist in the new ref skeleton, then clear out the bone name.
			if (!BindingBoneName.BoneName.IsNone())
			{
				const FReferenceSkeleton* RefSkeleton = nullptr;
				if (SkeletonImportOption == EReferenceSkeletonImportOption::UseExistingSkeleton)
				{
					const USkeleton* SkeletonPtr = Cast<USkeleton>(Skeleton.ResolveObject());
					RefSkeleton = SkeletonPtr ? &SkeletonPtr->GetReferenceSkeleton() : nullptr;  
				}
				else if (SkeletonImportOption == EReferenceSkeletonImportOption::UseExistingSkeletalMesh)
				{
					const USkeletalMesh* SkeletalMeshPtr = Cast<USkeletalMesh>(SkeletalMesh.ResolveObject());
					RefSkeleton = SkeletalMeshPtr ? &SkeletalMeshPtr->GetRefSkeleton() : nullptr;
				}

				if (RefSkeleton && RefSkeleton->FindBoneIndex(BindingBoneName.BoneName) == INDEX_NONE)
				{
					BindingBoneName.Reset();
				}
			}
		}
	}
}

static void CreateAssetName(
	const UPackage* InTemplatePackageName,
	const FString& InTargetPackagePath,
	const FString& InPrefixToRemove,
	const FString& InPrefixToAdd,
	const FString& InSuffixToAdd,
	FString &OutNewAssetName,
	FString& OutNewPackageName
	)
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	
	FString TemplatePackageName = InTemplatePackageName->GetName();
	FString TemplateAssetName = FPackageName::GetLongPackageAssetName(TemplatePackageName);
	TemplateAssetName.RemoveFromStart(InPrefixToRemove);
	TemplateAssetName.InsertAt(0, InPrefixToAdd);

	TemplatePackageName = InTargetPackagePath + TEXT("/") + TemplateAssetName; 
	
	AssetTools.CreateUniqueAssetName(TemplatePackageName, InSuffixToAdd, OutNewPackageName, OutNewAssetName);
}

static bool ConvertSingleMeshToSkeletalMesh(
	const UStaticMeshToSkeletalMeshConvertOptions* InOptions,
	UStaticMesh* InStaticMesh,
	TArray<UObject*>& OutObjectsAdded
	)
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	FReferenceSkeleton ReferenceSkeleton;
	USkeleton* Skeleton = nullptr;
	FName BindBoneName;

	switch(InOptions->SkeletonImportOption)
	{
	case EReferenceSkeletonImportOption::CreateNew:
		{
			FString SkeletonName;
			FString SkeletonPackageName;

			CreateAssetName(InStaticMesh->GetPackage(), InOptions->DestinationPath.Path, InOptions->PrefixToRemove, InOptions->SkeletonPrefixToAdd, InOptions->SkeletonSuffixToAdd, SkeletonName, SkeletonPackageName);

			USkeletonFromStaticMeshFactory* SkeletonFactory = NewObject<USkeletonFromStaticMeshFactory>();
			SkeletonFactory->StaticMesh = InStaticMesh;

			switch(InOptions->RootBonePlacement)
			{
			case ERootBonePlacementOptions::BottomCenter:
				SkeletonFactory->RootPosition = FVector(0.5, 0.5, 0.0);
				SkeletonFactory->PositionReference = ERootBonePositionReference::Relative;
				break;
			case ERootBonePlacementOptions::Center:
				SkeletonFactory->RootPosition = FVector(0.5, 0.5, 0.5);
				SkeletonFactory->PositionReference = ERootBonePositionReference::Relative;
				break;
			case ERootBonePlacementOptions::Origin:
				SkeletonFactory->RootPosition = FVector(0.0, 0.0, 0.0);
				SkeletonFactory->PositionReference = ERootBonePositionReference::Absolute;
				break;
			}

			Skeleton = Cast<USkeleton>(AssetTools.CreateAsset(SkeletonName, FPackageName::GetLongPackagePath(SkeletonPackageName), USkeleton::StaticClass(), SkeletonFactory));
			if (!Skeleton)
			{
				return false;
			}
			ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
			
			OutObjectsAdded.Add(Skeleton);
		}
		break;
		
	case EReferenceSkeletonImportOption::UseExistingSkeleton:
		{
			Skeleton = Cast<USkeleton>(InOptions->Skeleton.ResolveObject());
			if (!Skeleton)
			{
				UE_LOGF(LogSkeletalMeshModelingToolsMeshConverter, Error, "No valid skeleton given as a reference. Aborting conversion.");
				return false;
			}
			ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
			BindBoneName = InOptions->BindingBoneName.BoneName;
		}
		break;
	case EReferenceSkeletonImportOption::UseExistingSkeletalMesh:
		{
			USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InOptions->SkeletalMesh.ResolveObject());
			if (!SkeletalMesh)
			{
				UE_LOGF(LogSkeletalMeshModelingToolsMeshConverter, Error, "No valid skeletal mesh given as reference. Aborting conversion.");
				return false;
			}
			Skeleton = SkeletalMesh->GetSkeleton();
			ReferenceSkeleton = SkeletalMesh->GetRefSkeleton();
			BindBoneName = InOptions->BindingBoneName.BoneName;
		}
		break;
	}

	USkeletalMesh *SkeletalMesh = nullptr;
	if (ensure(Skeleton) && ensure(ReferenceSkeleton.GetNum() != 0))
	{
		FString SkeletalMeshName;
		FString SkeletalMeshPackageName;
		
		CreateAssetName(InStaticMesh->GetPackage(), InOptions->DestinationPath.Path, InOptions->PrefixToRemove, InOptions->SkeletalMeshPrefixToAdd, InOptions->SkeletalMeshSuffixToAdd, SkeletalMeshName, SkeletalMeshPackageName);

		USkeletalMeshFromStaticMeshFactory* SkeletalMeshFactory = NewObject<USkeletalMeshFromStaticMeshFactory>();
		SkeletalMeshFactory->StaticMesh = InStaticMesh;
		SkeletalMeshFactory->ReferenceSkeleton = ReferenceSkeleton;
		SkeletalMeshFactory->Skeleton = Skeleton;
		SkeletalMeshFactory->BindBoneName = BindBoneName; 

		SkeletalMesh = Cast<USkeletalMesh>(AssetTools.CreateAsset(SkeletalMeshName, FPackageName::GetLongPackagePath(SkeletalMeshPackageName), USkeletalMesh::StaticClass(), SkeletalMeshFactory));
	}

	if (ensure(SkeletalMesh))
	{
		OutObjectsAdded.Add(SkeletalMesh);
		return true;
	}

	return false;
}

static bool ConvertMultipleMeshesToSkeletalMesh(
	UStaticMeshToSkeletalMeshConvertOptions* InOptions,
	TArray<UStaticMesh*> InStaticMeshes,
	TArray<UObject*>& OutObjectsAdded
	)
{
	for (UStaticMesh* StaticMesh: InStaticMeshes)
	{
		if (!ConvertSingleMeshToSkeletalMesh(InOptions, StaticMesh, OutObjectsAdded))
		{
			return false;
		}
	}
	return true;
}

static void DoConversion(
	UStaticMeshToSkeletalMeshConvertOptions* InOptions,
	const TArray<UStaticMesh*>& InMeshesToConvert
	)
{
	auto IsValidPathPart = [](const FString& InPathPart, FStringView InPathPartName, const FText& InMessage)-> bool
	{
		FText FailureReason;
		const FText FailureContext = FText::FromStringView(InPathPartName);
		if (!FName::IsValidXName(InPathPart, INVALID_OBJECTNAME_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &FailureReason, &FailureContext))
		{
			ShowEditorMessage(ELogVerbosity::Error, InMessage, &FailureReason); 
			return false;
		}
		return true;
	};

	FPackageName::EErrorCode PackagePathErrorCode;
	if (!FPackageName::IsValidLongPackageName(InOptions->DestinationPath.Path, false, &PackagePathErrorCode))
	{
		const FText LogMessage = FPackageName::FormatErrorAsText(InOptions->DestinationPath.Path, PackagePathErrorCode);
		ShowEditorMessage(ELogVerbosity::Error, LOCTEXT("InvalidDestinationPath", "Destination Path is Invalid"), &LogMessage); 
		return;
	}

	if (!IsValidPathPart(InOptions->SkeletalMeshPrefixToAdd, TEXT("Skeletal Mesh Prefix"), LOCTEXT("InvalidSkeletalMeshPrefix", "Invalid Characters in Skeletal Mesh Prefix")) ||
		!IsValidPathPart(InOptions->SkeletalMeshSuffixToAdd, TEXT("Skeletal Mesh Suffix"), LOCTEXT("InvalidSkeletalMeshSuffix", "Invalid Characters in Skeletal Mesh Suffix")))
	{	
		return;
	}
				
	if (InOptions->SkeletonImportOption == EReferenceSkeletonImportOption::CreateNew)
	{
		if (!IsValidPathPart(InOptions->SkeletonPrefixToAdd, TEXT("Skeleton Prefix"), LOCTEXT("InvalidSkeletonPrefix", "Invalid Characters in Skeleton Prefix")) ||
			!IsValidPathPart(InOptions->SkeletonSuffixToAdd, TEXT("Skeleton Suffix"), LOCTEXT("InvalidSkeletonSuffix", "Invalid Characters in Skeleton Suffix")))
		{
			return;
		}
	}

	InOptions->SaveConfig();
	
	TArray<UObject*> ObjectsAdded;
	if (ConvertMultipleMeshesToSkeletalMesh(InOptions, InMeshesToConvert, ObjectsAdded))
	{
		FAssetToolsModule::GetModule().Get().SyncBrowserToAssets(ObjectsAdded);
	}
	else
	{
		for (UObject* ObjectToDelete: ObjectsAdded)
		{
			FAssetRegistryModule::AssetDeleted(ObjectToDelete);
			ObjectToDelete->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
		}
	}
}

void ConvertStaticMeshAssetsToSkeletalMeshesInteractive(
	const TArray<FAssetData>& InStaticMeshAssets
	)
{
	TArray<UStaticMesh*> MeshesToConvert;

	for (const FAssetData& AssetData: InStaticMeshAssets)
	{
		UStaticMesh* StaticMesh = Cast<UStaticMesh>(AssetData.GetAsset());
		if (!ensure(StaticMesh))
		{
			continue;
		}

		MeshesToConvert.Add(StaticMesh);
	}
	if (MeshesToConvert.IsEmpty())
	{
		return;
	}

	UStaticMeshToSkeletalMeshConvertOptions* Options = NewObject<UStaticMeshToSkeletalMeshConvertOptions>();
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	// If the object paths don't resolve properly, the details view will still show something, which will be invalidated
	// later.
	if (Cast<USkeleton>(Options->Skeleton.ResolveObject()) == nullptr)
	{
		Options->Skeleton.Reset();
	}
	if (Cast<USkeletalMesh>(Options->SkeletalMesh.ResolveObject()) == nullptr)
	{
		Options->SkeletalMesh.Reset();
	}

	Options->DestinationPath.Path = FPackageName::GetLongPackagePath(MeshesToConvert[0]->GetPackage()->GetPathName());
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(Options, /*bForceRefresh*/true);

	TSharedRef<SCustomDialog> OptionsDialog = SNew(SCustomDialog)
		.Title(LOCTEXT("OptionsDialogTitle", "Static Mesh Conversion Options"))
		.Content()
		[
			SNew(SBox)
			.MinDesiredWidth(450)
			[
				DetailsView
			]
		]
	.Buttons({
		SCustomDialog::FButton(LOCTEXT("DialogButtonConvert", "Convert"), FSimpleDelegate::CreateLambda([Options, MeshesToConvert]()
		{
			DoConversion(Options, MeshesToConvert);
		})),
		SCustomDialog::FButton(LOCTEXT("DialogButtonCancel", "Cancel"))
	});

	OptionsDialog->Show();
}

#undef LOCTEXT_NAMESPACE
