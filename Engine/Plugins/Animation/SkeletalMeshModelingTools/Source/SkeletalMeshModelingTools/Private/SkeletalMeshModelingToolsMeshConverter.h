// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneContainer.h"
#include "Factories/Factory.h"
#include "ReferenceSkeleton.h"
#include "Animation/BoneSocketReference.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"

#include "SkeletalMeshModelingToolsMeshConverter.generated.h"


class USkeleton;
class UStaticMesh;
struct FAssetData;

UENUM()
enum class ERootBonePositionReference
{
	Relative,
	Absolute
};


UCLASS(Hidden, MinimalAPI)
class USkeletonFromStaticMeshFactory :
	public UFactory
{
	GENERATED_BODY()
public:
	USkeletonFromStaticMeshFactory(const FObjectInitializer& InObjectInitializer);

	// UFactory overrides
	virtual UObject* FactoryCreateNew(
		UClass* InClass,
		UObject* InParent,
		FName InName,
		EObjectFlags InFlags,
		UObject* InContext,
		FFeedbackContext* InWarn
		) override;

	virtual bool ShouldShowInNewMenu() const override { return false; }
	
	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh;

	UPROPERTY()
	FVector RootPosition = {0.5, 0.5, 0.0};

	UPROPERTY()
	ERootBonePositionReference PositionReference = ERootBonePositionReference::Relative;
};


UCLASS(Hidden, MinimalAPI)
class USkeletalMeshFromStaticMeshFactory :
	public UFactory
{
	GENERATED_BODY()
public:
	USkeletalMeshFromStaticMeshFactory(const FObjectInitializer& InObjectInitializer);

	// UFactory overrides
	virtual UObject* FactoryCreateNew(
		UClass* InClass,
		UObject* InParent,
		FName InName,
		EObjectFlags InFlags,
		UObject* InContext,
		FFeedbackContext* InWarn
		) override;
	
	virtual bool ShouldShowInNewMenu() const override { return false; }

	FReferenceSkeleton ReferenceSkeleton;

	UPROPERTY()
	TObjectPtr<USkeleton> Skeleton;
	
	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh;

	UPROPERTY()
	FName BindBoneName;
};


UENUM()
enum class EReferenceSkeletonImportOption
{
	CreateNew UMETA(ToolTip="Create a new skeleton with a single root bone"),
	UseExistingSkeleton UMETA(ToolTip="Use an already created skeleton asset"),
	UseExistingSkeletalMesh UMETA(ToolTip="Use a skeleton use by another skeletal mesh asset")
};

UENUM()
enum class ERootBonePlacementOptions
{
	BottomCenter UMETA(DisplayName="Bottom center", ToolTip="Bottom-center of the static mesh's bounding box"),
	Center UMETA(DisplayName="Center", ToolTip="Center of the static mesh's bounding box"),
	Origin UMETA(DisplayName="Origin", ToolTip="At the static mesh's origin (0,0,0)"),
};



UCLASS(config=EditorPerProjectUserSettings, HideCategories=Object, MinimalAPI)
class UStaticMeshToSkeletalMeshConvertOptions :
	public UObject,
	public IBoneReferenceSkeletonProvider
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Config, Category="Location", meta=(ContentDir))
	FDirectoryPath DestinationPath;
	

	UPROPERTY(EditAnywhere, Config, Category="Skeleton", meta=(DisplayName="Creation Type"))
	EReferenceSkeletonImportOption SkeletonImportOption = EReferenceSkeletonImportOption::CreateNew;

	UPROPERTY(EditAnywhere, Config, Category="Skeleton", meta=(EditCondition="SkeletonImportOption==EReferenceSkeletonImportOption::CreateNew", EditConditionHides))
	ERootBonePlacementOptions RootBonePlacement = ERootBonePlacementOptions::BottomCenter;
	
	UPROPERTY(EditAnywhere, Config, Category="Skeleton", meta=(AllowedClasses="/Script/Engine.Skeleton", EditCondition="SkeletonImportOption==EReferenceSkeletonImportOption::UseExistingSkeleton", EditConditionHides))
	FSoftObjectPath Skeleton;

	UPROPERTY(EditAnywhere, Config, Category="Skeleton", meta=(AllowedClasses="/Script/Engine.SkeletalMesh", EditCondition="SkeletonImportOption==EReferenceSkeletonImportOption::UseExistingSkeletalMesh", EditConditionHides))
	FSoftObjectPath SkeletalMesh;

	UPROPERTY(EditAnywhere, Config, Category="Skeleton", meta=(EditCondition="SkeletonImportOption!=EReferenceSkeletonImportOption::CreateNew", EditConditionHides))
	FBoneReference BindingBoneName;

	UPROPERTY(EditAnywhere, Config, Category="Naming")
	FString PrefixToRemove = TEXT("SM_");
	
	UPROPERTY(EditAnywhere, Config, Category="Naming", meta=(DisplayName="Skeletal Mesh Prefix"))
	FString SkeletalMeshPrefixToAdd = TEXT("SKM_");

	UPROPERTY(EditAnywhere, Config, Category="Naming", meta=(DisplayName="Skeletal Mesh Suffix"))
	FString SkeletalMeshSuffixToAdd;
	
	UPROPERTY(EditAnywhere, Config, Category="Naming", meta=(DisplayName="Skeleton Prefix", EditCondition="SkeletonImportOption==EReferenceSkeletonImportOption::CreateNew"))
	FString SkeletonPrefixToAdd = TEXT("SK_");

	UPROPERTY(EditAnywhere, Config, Category="Naming", meta=(DisplayName="Skeleton Suffix", EditCondition="SkeletonImportOption==EReferenceSkeletonImportOption::CreateNew"))
	FString SkeletonSuffixToAdd;
	
	// IBoneReferenceSkeletonProvider implementation.
	virtual USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;
	
	// UObject overrides.
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};


/** Converts a static mesh to a skeletal mesh, creating both a skeletal mesh and a skeleton assets that link up. */
void ConvertStaticMeshAssetsToSkeletalMeshesInteractive(const TArray<FAssetData>& InStaticMeshAssets);
