// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshModelingToolsModule.h"

#include "SkeletalMeshModelingToolsCommands.h"
#include "SkeletalMeshModelingToolsEditorMode.h"
#include "SkeletalMeshModelingToolsMeshConverter.h"
#include "SkeletalMeshModelingToolsStyle.h"

#include "ContentBrowserMenuContexts.h"
#include "EditorModeManager.h"
#include "IPersonaToolkit.h"
#include "Engine/SkeletalMesh.h"
#include "ISkeletalMeshEditor.h"
#include "ISkeletalMeshEditorModule.h"
#include "ModelingToolsEditorModeStyle.h"
#include "ModelingToolsManagerActions.h"
#include "PersonaModule.h"
#include "Modules/ModuleManager.h"
#include "SkeletalMeshToolMenuContext.h"
#include "ToolMenus.h"
#include "DetailCustomization/SkeletonEditingToolPropertyCustomizations.h"
#include "DetailCustomization/SkinWeightDetailCustomization.h"
#include "Misc/ConfigCacheIni.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"
#include "Styling/SlateIconFinder.h"
#include "WorkflowOrientedApp/ApplicationMode.h"


#define LOCTEXT_NAMESPACE "SkeletalMeshModelingToolsModule"

DEFINE_LOG_CATEGORY(LogSkeletalMeshModelingTools);

IMPLEMENT_MODULE(FSkeletalMeshModelingToolsModule, SkeletalMeshModelingTools);

static const TCHAR* ConfigSection = TEXT("SkeletalMeshModelingTools");
static const TCHAR* ConfigEditingModeActiveKey = TEXT("EditingModeActive");



void FSkeletalMeshModelingToolsModule::StartupModule()
{
	FSkeletalMeshModelingToolsStyle::Register();
	FSkeletalMeshModelingToolsCommands::Register();

	UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FSkeletalMeshModelingToolsModule::RegisterMenusAndToolbars));

	ISkeletalMeshEditorModule& SkelMeshEditorModule = FModuleManager::Get().LoadModuleChecked<ISkeletalMeshEditorModule>("SkeletalMeshEditor");
	// register toolbar extender with skeletal mesh editor
	TArray<ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender>& ToolbarExtenders = SkelMeshEditorModule.GetAllSkeletalMeshEditorToolbarExtenders();
	ToolbarExtenders.Add(ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender::CreateRaw(this, &FSkeletalMeshModelingToolsModule::ExtendSkelMeshEditorToolbar));
	SkelMeshEditorExtenderHandle = ToolbarExtenders.Last().GetHandle();
	// register post-init callback with skeletal mesh editor
	TArray<ISkeletalMeshEditorModule::FOnSkeletalMeshEditorInitialized>& PostInitDelegates = SkelMeshEditorModule.GetPostEditorInitDelegates();
	PostInitDelegates.Add(ISkeletalMeshEditorModule::FOnSkeletalMeshEditorInitialized::CreateRaw(this, &FSkeletalMeshModelingToolsModule::CheckEnableEditingToolModeOnOpen));
	SkelMeshEditorPostInitHandle = PostInitDelegates.Last().GetHandle();

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FSkeletalMeshModelingToolsModule::OnPostEngineInit);

	RegisterPropertyCustomizations();
}

void FSkeletalMeshModelingToolsModule::ShutdownModule()
{
	UnregisterPropertyCustomizations();
	
	if (ISkeletalMeshEditorModule* SkelMeshEditorModule = FModuleManager::GetModulePtr<ISkeletalMeshEditorModule>("SkeletalMeshEditor"))
	{
		// un-register toolbar extender delegates
		TArray<ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender>& Extenders = SkelMeshEditorModule->GetAllSkeletalMeshEditorToolbarExtenders();
		Extenders.RemoveAll([this](const ISkeletalMeshEditorModule::FSkeletalMeshEditorToolbarExtender& InDelegate) { return InDelegate.GetHandle() == SkelMeshEditorExtenderHandle; });

		// un-register post-init delegates
		TArray<ISkeletalMeshEditorModule::FOnSkeletalMeshEditorInitialized>& PostInitDelegates = SkelMeshEditorModule->GetPostEditorInitDelegates();
		PostInitDelegates.RemoveAll([this](const ISkeletalMeshEditorModule::FOnSkeletalMeshEditorInitialized& InDelegate) { return InDelegate.GetHandle() == SkelMeshEditorPostInitHandle; });
	}

	UToolMenus::UnregisterOwner(this);

	FSkeletalMeshModelingToolsCommands::Unregister();
	FSkeletalMeshModelingToolsStyle::Unregister();
	FModelingToolsManagerCommands::Unregister();
	
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FSkeletalMeshModelingToolsActionCommands::UnregisterAllToolActions();
}


void FSkeletalMeshModelingToolsModule::RegisterMenusAndToolbars()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	if (UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("AssetEditor.SkeletalMeshEditor.ToolBar"))
	{
		FToolMenuSection& Section = ToolMenu->FindOrAddSection("SkeletalMesh");
		Section.AddDynamicEntry("ToggleModelingToolsMode", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection) {
			const USkeletalMeshToolMenuContext* Context = InSection.FindContext<USkeletalMeshToolMenuContext>();
			if (Context && Context->SkeletalMeshEditor.IsValid())
			{
				InSection.AddEntry(FToolMenuEntry::InitToolBarButton(
				    FSkeletalMeshModelingToolsCommands::Get().ToggleEditingToolsMode,
					LOCTEXT("SkeletalMeshEditorModelingMode", "Editing Tools"),
				    LOCTEXT("SkeletalMeshEditorModelingModeTooltip", "Opens the Editing Tools palette that provides selected skeletal mesh modification tools, including skeleton, skin weights and attributes editing."),
				    FSlateIcon("ModelingToolsStyle", "LevelEditor.ModelingToolsMode")));
			}
		}));
	}

	// Extend the asset browser for static meshes to include the conversion to skelmesh.
	if (UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.StaticMesh"))
	{
		FToolMenuSection& Section = ToolMenu->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry("ConvertToSkeletalMesh", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
			if (Context && Context->bCanBeModified)
			{
				const TAttribute<FText> Label = LOCTEXT("ConvertToSkeletalMesh", "Convert to Skeletal Mesh");
				const TAttribute<FText> ToolTip = LOCTEXT("ConvertToSkeletalMeshToolTip", "Converts a non-Nanite static mesh to a skeletal mesh with an associated skeleton.");
				const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(USkeletalMesh::StaticClass());
				const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InMenuContext)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = InMenuContext.FindContext<UContentBrowserAssetContextMenuContext>())
					{
						ConvertStaticMeshAssetsToSkeletalMeshesInteractive(
							Context->GetSelectedAssetsOfType(UStaticMesh::StaticClass()));
					}
				});

				InSection.AddMenuEntry("ConvertToSkeletalMesh", Label, ToolTip, Icon, UIAction);
			}
		}));
	}
}


TSharedRef<FExtender> FSkeletalMeshModelingToolsModule::ExtendSkelMeshEditorToolbar(const TSharedRef<FUICommandList> InCommandList, TSharedRef<ISkeletalMeshEditor> InSkeletalMeshEditor)
{
	// Add toolbar extender
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	TWeakPtr<ISkeletalMeshEditor> EditorPtr(InSkeletalMeshEditor);

	InCommandList->MapAction(FSkeletalMeshModelingToolsCommands::Get().ToggleEditingToolsMode,
	    FExecuteAction::CreateRaw(this, &FSkeletalMeshModelingToolsModule::OnToggleEditingToolsMode, EditorPtr),
	    FCanExecuteAction::CreateRaw(this, &FSkeletalMeshModelingToolsModule::IsMeshEditable, EditorPtr),
	    FIsActionChecked::CreateRaw(this, &FSkeletalMeshModelingToolsModule::IsEditingToolModeActive, EditorPtr));

	return ToolbarExtender.ToSharedRef();
}


bool FSkeletalMeshModelingToolsModule::IsEditingToolModeActive(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const
{
	TSharedPtr<ISkeletalMeshEditor> SkeletalMeshEditor = InSkeletalMeshEditor.Pin();
	return SkeletalMeshEditor.IsValid() && SkeletalMeshEditor->GetEditorModeManager().IsModeActive(USkeletalMeshModelingToolsEditorMode::Id);
}


void FSkeletalMeshModelingToolsModule::OnToggleEditingToolsMode(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor)
{
	TSharedPtr<ISkeletalMeshEditor> SkeletalMeshEditor = InSkeletalMeshEditor.Pin();
	if (SkeletalMeshEditor.IsValid())
	{
		FEditorModeTools& EditorModeManager = SkeletalMeshEditor->GetEditorModeManager();
		if (!IsEditingToolModeActive(InSkeletalMeshEditor))
		{
			EditorModeManager.ActivateMode(USkeletalMeshModelingToolsEditorMode::Id, true);

			// bind notifications to skeletal mesh modeling editor mode
			UEdMode* ActiveMode = EditorModeManager.GetActiveScriptableMode(USkeletalMeshModelingToolsEditorMode::Id);
			if (USkeletalMeshModelingToolsEditorMode* SkeletalMeshMode = CastChecked<USkeletalMeshModelingToolsEditorMode>(ActiveMode))
			{
				SkeletalMeshMode->SetEditorBinding(SkeletalMeshEditor);
			}
		}
		else
		{
			EditorModeManager.DeactivateMode(USkeletalMeshModelingToolsEditorMode::Id);
		}

		// Update the stored state of the editing tools active state.
		GConfig->SetBool(ConfigSection, ConfigEditingModeActiveKey, IsEditingToolModeActive(InSkeletalMeshEditor), GEditorPerProjectIni);

		// make sure SkeletonSelection is active when toggling the mode, as they are compatible.
		// it will be deactivated later when entering a tool 
		if (!EditorModeManager.IsModeActive(FPersonaEditModes::SkeletonSelection))
		{
			EditorModeManager.ActivateMode(FPersonaEditModes::SkeletonSelection);
		}
	}
}

void FSkeletalMeshModelingToolsModule::CheckEnableEditingToolModeOnOpen(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor)
{
	bool bEditingModeActive = true;
	GConfig->GetBool(ConfigSection, ConfigEditingModeActiveKey, bEditingModeActive, GEditorPerProjectIni);
	
	if (bEditingModeActive && !IsEditingToolModeActive(InSkeletalMeshEditor) && IsMeshEditable(InSkeletalMeshEditor))
	{
		OnToggleEditingToolsMode(InSkeletalMeshEditor);
	}
}

bool FSkeletalMeshModelingToolsModule::IsMeshEditable(TWeakPtr<ISkeletalMeshEditor> InSkeletonEditor)
{
	// Fetch mesh to check if it has a mesh description, we can crash if we do not.
	TSharedPtr<ISkeletalMeshEditor> SkeletalMeshEditor = InSkeletonEditor.Pin();
	if (!SkeletalMeshEditor.IsValid())
	{
		return false;
	}
	
	if (USkeletalMesh* Mesh = SkeletalMeshEditor->GetPersonaToolkit()->GetMesh())
	{
		return Mesh->HasMeshDescription(0);
	}
	return false;
}

void FSkeletalMeshModelingToolsModule::OnPostEngineInit()
{
	FSkeletalMeshModelingToolsActionCommands::RegisterAllToolActions();
	FModelingToolsEditorModeStyle::Initialize();
	FModelingToolsManagerCommands::Register();
}

void FSkeletalMeshModelingToolsModule::RegisterPropertyCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	auto RegisterPropertyCustomization = [&](FName InStructName, auto InCustomizationFactory)
	{
		PropertyModule.RegisterCustomPropertyTypeLayout(
			InStructName, 
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(InCustomizationFactory)
			);
		CustomizedProperties.Add(InStructName);
	};

	auto RegisterDetailCustomization = [&](FName InStructName, auto InCustomizationFactory)
	{
		PropertyModule.RegisterCustomClassLayout(
			InStructName,
			FOnGetDetailCustomizationInstance::CreateStatic(InCustomizationFactory)
		);
		CustomizedClasses.Add(InStructName);
	};

	
	RegisterDetailCustomization(USkinWeightsPaintToolProperties::StaticClass()->GetFName(), &FSkinWeightDetailCustomization::MakeInstance);
	RegisterDetailCustomization(USkeletonEditingTool::StaticClass()->GetFName(), &FSkeletonEditingToolDetailCustomization::MakeInstance);
	RegisterDetailCustomization(USkeletonEditingProperties::StaticClass()->GetFName(), &FSkeletonEditingPropertiesDetailCustomization::MakeInstance);
	RegisterDetailCustomization(UMirroringProperties::StaticClass()->GetFName(), &FMirroringPropertiesDetailCustomization::MakeInstance);
	RegisterDetailCustomization(UOrientingProperties::StaticClass()->GetFName(), &FOrientingPropertiesDetailCustomization::MakeInstance);
	RegisterDetailCustomization(UProjectionProperties::StaticClass()->GetFName(), &FProjectionPropertiesDetailCustomization::MakeInstance);

	PropertyModule.NotifyCustomizationModuleChanged();
}

void FSkeletalMeshModelingToolsModule::UnregisterPropertyCustomizations()
{
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (const FName& PropertyName: CustomizedProperties)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
		for (const FName& ClassName : CustomizedClasses)
		{
			PropertyModule->UnregisterCustomClassLayout(ClassName);
		}
		PropertyModule->NotifyCustomizationModuleChanged();
	}
}

#undef LOCTEXT_NAMESPACE
