// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"


DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMeshModelingTools, Log, All);


class FApplicationMode;
class ISkeletalMeshEditor;
class FUICommandList;
class FExtender;

class FSkeletalMeshModelingToolsModule : 
	public IModuleInterface
{
public:
	// IModuleInterface implementations
	void StartupModule() override;
	void ShutdownModule() override;

private:
	// Extend toolbars and menus
	void RegisterMenusAndToolbars();

	TSharedRef<FExtender> ExtendSkelMeshEditorToolbar(const TSharedRef<FUICommandList> InCommandList, TSharedRef<ISkeletalMeshEditor> InSkeletalMeshEditor);


	bool IsEditingToolModeActive(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor) const;
	void OnToggleEditingToolsMode(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor);
	void CheckEnableEditingToolModeOnOpen(TWeakPtr<ISkeletalMeshEditor> InSkeletalMeshEditor);
	bool IsMeshEditable(TWeakPtr<ISkeletalMeshEditor> InSkeletonEditor);

	void OnPostEngineInit();
	
	void RegisterPropertyCustomizations();
	void UnregisterPropertyCustomizations();

	TArray<FName> CustomizedProperties;
	TArray<FName> CustomizedClasses;
	
	// The handle for the extender delegate we added. Needed for clean module shutdown.
	FDelegateHandle SkelMeshEditorExtenderHandle;
	FDelegateHandle SkelMeshEditorPostInitHandle;

	TArray<TWeakPtr<FApplicationMode>> RegisteredApplicationModes;
};
