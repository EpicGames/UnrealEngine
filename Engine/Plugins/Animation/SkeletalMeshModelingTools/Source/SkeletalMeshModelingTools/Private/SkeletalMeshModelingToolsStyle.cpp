// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshModelingToolsStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Styling/CoreStyle.h"

FSkeletalMeshModelingToolsStyle::FSkeletalMeshModelingToolsStyle()
	: FSlateStyleSet("SkeletalMeshModelingToolsStyle")
{
	// Core content root used by RootToCoreContentDir for engine-wide icon lookups.
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FVector2D Icon20x20(20.0f, 20.0f);
	Set("SkeletalMeshModelingTools.BeginSkeletalMeshRunMeshProcessorBlueprintTool",
		new FSlateVectorImageBrush(RootToCoreContentDir(TEXT("../Editor/Slate/Starship/AssetIcons/Blueprint_64.svg")), Icon20x20));
	Set("SkeletalMeshModelingTools.BeginSkeletalMeshRunMeshProcessorBlueprintTool.Small",
		new FSlateVectorImageBrush(RootToCoreContentDir(TEXT("../Editor/Slate/Starship/AssetIcons/Blueprint_64.svg")), Icon20x20));

	// Misc palette icon — borrow the SVG from ModelingToolsEditorMode's LoadLodsTools so our
	// Misc tab reads consistently with the equivalent palette in regular Modeling Mode.
	if (TSharedPtr<IPlugin> ModelingToolsEditorModePlugin = IPluginManager::Get().FindPlugin(TEXT("ModelingToolsEditorMode")))
	{
		const FString MiscIconPath = ModelingToolsEditorModePlugin->GetContentDir() / TEXT("Icons/LoadLodsTools.svg");
		Set("SkeletalMeshModelingTools.LoadMiscTools",
			new FSlateVectorImageBrush(MiscIconPath, Icon20x20));
		Set("SkeletalMeshModelingTools.LoadMiscTools.Small",
			new FSlateVectorImageBrush(MiscIconPath, Icon20x20));
	}

	FCheckBoxStyle BaseCheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
	Set("SkeletalMeshModelingTools.ToggleButtonCheckbox",
		BaseCheckBoxStyle
		.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f))
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Input, 1.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
		.SetPadding(FMargin(16, 6))
	);

	// Split accept button styles based on default "Button" style.
	// No outlines — matches viewport toolbar split button approach to avoid double borders at seams.
	{
		const float Radius = 4.0f;
		const FVector4 StartCorners(Radius, 0.f, 0.f, Radius);
		const FVector4 EndCorners(0.f, Radius, Radius, 0.f);
		const FVector4 AllCorners(Radius, Radius, Radius, Radius);

		const FButtonStyle& BaseButton = FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button");

		// "Apply to Asset" button (green) — matches FlatButton.Success colors
		// Color multipliers from StarshipStyle ButtonColor: Normal=0.8, Hovered=1.0, Pressed=0.6
		{
			const FLinearColor SuccessBase(0.10616f, 0.48777f, 0.10616f);
			FLinearColor SuccessNormal = SuccessBase * 0.8f; SuccessNormal.A = 1.0f;
			FLinearColor SuccessHovered = SuccessBase * 1.0f; SuccessHovered.A = 1.0f;
			FLinearColor SuccessPressed = SuccessBase * 0.6f; SuccessPressed.A = 1.0f;

			Set("SkeletalMeshModelingTools.AcceptButton.ApplyToAsset",
				FButtonStyle(BaseButton)
				.SetNormal(FSlateRoundedBoxBrush(SuccessNormal, StartCorners))
				.SetHovered(FSlateRoundedBoxBrush(SuccessHovered, StartCorners))
				.SetPressed(FSlateRoundedBoxBrush(SuccessPressed, StartCorners))
				.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, StartCorners))
			);
		}

		// "Exit Tool" button (blue) — matches PrimaryButton colors via FStyleColors
		{
			Set("SkeletalMeshModelingTools.AcceptButton.ExitTool",
				FButtonStyle(BaseButton)
				.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Primary, StartCorners))
				.SetHovered(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, StartCorners))
				.SetPressed(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, StartCorners))
				.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, StartCorners))
			);
		}

		// Combo dropdown button (right side) — default button colors with rounded right corners
		{
			Set("SkeletalMeshModelingTools.AcceptButton.ComboButton",
				FComboButtonStyle(FCoreStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
				.SetButtonStyle(
					FButtonStyle(BaseButton)
					.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, EndCorners))
					.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, EndCorners))
					.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, EndCorners))
					.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, EndCorners))
					.SetNormalPadding(0.f)
					.SetPressedPadding(0.f)
				)
				.SetDownArrowPadding(FMargin(2.0f))
			);
		}

		// Cancel button — default button colors, fully rounded, no outline
		{
			Set("SkeletalMeshModelingTools.CancelButton",
				FButtonStyle(BaseButton)
				.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, AllCorners))
				.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, AllCorners))
				.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, AllCorners))
				.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, AllCorners))
			);
		}
	}
}

FSkeletalMeshModelingToolsStyle& FSkeletalMeshModelingToolsStyle::Get()
{
	static FSkeletalMeshModelingToolsStyle Instance;
	return Instance;
}

void FSkeletalMeshModelingToolsStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FSkeletalMeshModelingToolsStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}
