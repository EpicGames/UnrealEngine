// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshModelingToolsViewportToolbarExtension.h"

#include "Editor/Persona/Private/AnimViewportShowCommands.h"
#include "Editor/Persona/Private/ViewportToolbar/AnimationEditorMakeShowBonesMenuSection.h"
#include "ISkeletalMeshEditor.h"
#include "ModelingToolsEditorModeStyle.h"
#include "ModelingToolsManagerActions.h"
#include "Selection/GeometrySelectionManager.h"
#include "SkeletalMesh/SkeletalMeshModelingToolsEditorSettings.h"
#include "SkeletalMeshModelingToolsCommands.h"
#include "SkeletalMeshModelingToolsEditorMode.h"
#include "SkeletalMeshModelingToolsEditorModeToolkit.h"
#include "SkeletalMeshToolMenuContext.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Selection/GeometrySelector.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshModelingToolsViewportToolbarExtension"

namespace UE::SkeletalMeshModelingTools
{
	// Creates the skeletal mesh editor menu
	namespace SkeletalMeshEditorMenu
	{
		/** Creates a slider widget */
		TSharedRef<SWidget> CreateFloatSliderWidget(
			const FText& TooltipText,
			const float MaxValue,
			const TAttribute<TOptional<float>>& ValueAttribute,
			const SNumericEntryBox<float>::FOnValueChanged& OnValueChangedDelegate)
		{
			return 
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
					.WidthOverride(100.0f)
					[
						SNew(SNumericEntryBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.AllowSpin(true)
						.MinSliderValue(0.0f)
						.MaxSliderValue(MaxValue)
						.Value(ValueAttribute)
						.OnValueChanged(OnValueChangedDelegate)
						.ToolTipText(TooltipText)
					]
				];			
		}
		
		/** Creates a widget to edit the axis length */
		TSharedRef<SWidget> MakeAxisLengthWidget()
		{
			const FText Tooltip = LOCTEXT("SkeletalMeshEditorAxesLengthTooltip", "Sets the length of displayed local rotation axes");
			constexpr float MaxValue = 25.f;
			
			return SkeletalMeshEditorMenu::CreateFloatSliderWidget(
				Tooltip,
				MaxValue,
				TAttribute<TOptional<float>>::CreateLambda([]()
					{
						return GetDefault<USkeletalMeshModelingToolsEditorSettings>()->GetLocalRotationAxisLength();
					}),
				SNumericEntryBox<float>::FOnValueChanged::CreateLambda([](float Value)
					{
						USkeletalMeshModelingToolsEditorSettings* Settings = GetMutableDefault<USkeletalMeshModelingToolsEditorSettings>();
						Settings->SetLocalRotationAxisLength(FMath::Max(0.f, Value));
						Settings->SaveConfig();
					})
				);
		}
		
		/** Creates a widget to edit the axis thickness */
		TSharedRef<SWidget> MakeAxisThicknessWidget()
		{
			const FText Tooltip = LOCTEXT("SkeletalMeshEditorAxesThicknessTooltip", "Sets the thickness of displayed local rotation axes");
			constexpr float MaxValue = 1.f;

			return SkeletalMeshEditorMenu::CreateFloatSliderWidget(
				Tooltip,
				MaxValue,
				TAttribute<TOptional<float>>::CreateLambda([]()
					{
						return GetDefault<USkeletalMeshModelingToolsEditorSettings>()->GetLocalRotationAxisThickness();
					}),
				SNumericEntryBox<float>::FOnValueChanged::CreateLambda([](float Value)
					{
						USkeletalMeshModelingToolsEditorSettings* Settings = GetMutableDefault<USkeletalMeshModelingToolsEditorSettings>();
						Settings->SetLocalRotationAxisThickness(FMath::Max(0.f, Value));
						Settings->SaveConfig();
					})
				);
		}

		/** Adds the skeletal mesh menu to the toolbar */
		void CreateSkeletalMeshEditorViewportToolbarMenu(
			const TSharedRef<FUICommandList>& InCommandList, 
			UToolMenu& InOutPersonaViewportToolbar)
		{
			FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
				"SkeletalMeshEditor",
				LOCTEXT("SkeletalMeshEditorMenuLabel", "Skeletal Mesh"),
				LOCTEXT("SkeletalMeshEditorMenuTooltip", "Skeletal Mesh editor related settings"),
				FNewToolMenuDelegate::CreateLambda(
					[WeakCommandList = InCommandList.ToWeakPtr()](UToolMenu* SubMenu) -> void
					{
						if (const TSharedPtr<FUICommandList> CommandList = WeakCommandList.Pin())
						{
							// Local Rotation Axes section
							{
								FToolMenuSection& SkeletalMeshEditorSection =
									SubMenu->FindOrAddSection(
										"SkeletalMeshEditor",
										LOCTEXT("SkeletalMeshEditorLabel", "Local Rotation Axes"));

								const FSkeletalMeshModelingToolsCommands& Commands = FSkeletalMeshModelingToolsCommands::Get();

								SkeletalMeshEditorSection.AddMenuEntryWithCommandList(Commands.ToggleShowAllLocalRotationAxes, WeakCommandList.Pin());

								SkeletalMeshEditorSection.AddEntry(FToolMenuEntry::InitWidget(
									"AxisLength",
									MakeAxisLengthWidget(),
									LOCTEXT("SkeletalMeshEditorLocalRotationAxesLength", "Axes Length")));

								SkeletalMeshEditorSection.AddEntry(FToolMenuEntry::InitWidget(
									"AxisThickness",
									MakeAxisThicknessWidget(),
									LOCTEXT("SkeletalMeshEditorLocalRotationAxesThickness", "Axes Thickness")));
							}

							// Show bones section
							{
								using namespace UE::Persona::ViewportToolbar;

								const EShowBonesMenuEntryFlags ShowFlags =
									EShowBonesMenuEntryFlags::DrawAll |
									EShowBonesMenuEntryFlags::DrawNone;

								MakeShowBonesMenuSection(SubMenu, ShowFlags);
							}
						}
					})
				);
			
			Entry.InsertPosition = FToolMenuInsert("Show", EToolMenuInsertType::After);
			Entry.Icon = FSlateIconFinder::FindIconForClass(USkeletalMesh::StaticClass());
			Entry.ToolBarData.LabelOverride = FText::GetEmpty();

			// Set resize prio to highest as other entries in other sections dominate otherwise
			Entry.ToolBarData.ResizeParams.ClippingPriority = std::numeric_limits<int32>::max(); 

			FToolMenuSection* const RightSection = InOutPersonaViewportToolbar.FindSection("Right");
			if (ensureMsgf(RightSection, TEXT("Cannot find the right section in the persona viewport toolbar, cannot extend menu")))
			{
				RightSection->AddDynamicEntry(
					"SkeletalMeshEditor",
					FNewToolMenuSectionDelegate::CreateLambda([Entry](FToolMenuSection& Section)
						{
							// Only extending the menu for skeletal mesh editor
							const USkeletalMeshToolMenuContext* Context = Section.FindContext<USkeletalMeshToolMenuContext>();
							if (Context && Context->SkeletalMeshEditor.IsValid())
							{
								Section.AddEntry(Entry);
							}
						})
				);
			}
		}
	} //~ namespace SkeletalMeshEditorMenu

	// Extends the transform menu
	namespace TransformMenuExtension
	{
		void ExtendTransformMenu(
			USkeletalMeshModelingToolsEditorMode& InEdMode, 
			const TSharedRef<FUICommandList>& CommandList,
			UToolMenu& InOutPersonaViewportToolbar)
		{
			const TWeakObjectPtr<USkeletalMeshModelingToolsEditorMode> WeakEdMode = &InEdMode;
			
			FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
				"Mesh Element Selection",
				LOCTEXT("MeshElementSelectionSubmenuLabel", "Mesh Element Selection"),
				LOCTEXT("MeshElementSelectionSubmenuTooltip", "Mesh Element Selection settings in the viewport"),
				FNewToolMenuDelegate::CreateLambda([WeakEdMode, WeakCommandList = CommandList.ToWeakPtr()](UToolMenu* Submenu) -> void
					{
						FToolMenuSection& MeshElementSelectionOptionsSection =
							Submenu->FindOrAddSection("Element Selection", LOCTEXT("ElementSelectionLabel", "Element Selection"));
						if (const TSharedPtr<const FUICommandList>& CommandList = WeakCommandList.Pin())
						{
							MeshElementSelectionOptionsSection.AddEntry( FToolMenuEntry::InitMenuEntryWithCommandList(
								FSkeletalMeshModelingToolsCommands::Get().ShowQuickAccessMenu, CommandList));
							MeshElementSelectionOptionsSection.AddSeparator(NAME_None);
							
							auto CreateToolMenuEntry = [&CommandList, &MeshElementSelectionOptionsSection](const TSharedPtr<FUICommandInfo>& Command)
							{
								FToolMenuEntry MESMode = FToolMenuEntry::InitMenuEntryWithCommandList(Command, CommandList);
								MESMode.SetShowInToolbarTopLevel(true); // add button to toolbar
								MeshElementSelectionOptionsSection.AddEntry(MESMode);
							};
							
							// add all Mesh Element Selection modes
							CreateToolMenuEntry(FModelingToolsManagerCommands::Get().MeshSelectionModeAction_NoSelection);
							MeshElementSelectionOptionsSection.AddSeparator(NAME_None);
							CreateToolMenuEntry(FModelingToolsManagerCommands::Get().MeshSelectionModeAction_MeshVertices);
							CreateToolMenuEntry(FModelingToolsManagerCommands::Get().MeshSelectionModeAction_MeshEdges);
							CreateToolMenuEntry(FModelingToolsManagerCommands::Get().MeshSelectionModeAction_MeshTriangles);
							MeshElementSelectionOptionsSection.AddSeparator(NAME_None);
							CreateToolMenuEntry(FModelingToolsManagerCommands::Get().MeshSelectionModeAction_GroupCorners);
							CreateToolMenuEntry(FModelingToolsManagerCommands::Get().MeshSelectionModeAction_GroupEdges);
							CreateToolMenuEntry(FModelingToolsManagerCommands::Get().MeshSelectionModeAction_GroupFaces);
							
							// Section for Selection Edits
							{
								FToolMenuSection& SelectionEditsSection =
								Submenu->FindOrAddSection("SelectionEdits", LOCTEXT("SelectionEditsLabel", "Selection Edits"));
								
								SelectionEditsSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().BeginSelectionAction_SelectAll, CommandList));
								SelectionEditsSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().BeginSelectionAction_ExpandToConnected, CommandList));
								SelectionEditsSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().BeginSelectionAction_Invert, CommandList));
								SelectionEditsSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().BeginSelectionAction_InvertConnected, CommandList));
								SelectionEditsSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().BeginSelectionAction_Expand, CommandList));
								SelectionEditsSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().BeginSelectionAction_Contract, CommandList));
							}
							
							// Section for Local Frame Mode setting
							{
								FToolMenuSection& LocalFrameModeSection =
								Submenu->FindOrAddSection("Local Frame Mode", LOCTEXT("LocalFrameModeLabel", "Local Frame Mode"));
								LocalFrameModeSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FModelingToolsManagerCommands::Get().SelectionLocalFrameMode_Geometry, CommandList));
								LocalFrameModeSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FModelingToolsManagerCommands::Get().SelectionLocalFrameMode_Object, CommandList));
							}
							
							// Section for Selection Filters
							{
								FToolMenuSection& SelectionFilterSection =
								Submenu->FindOrAddSection("Selection Filters", LOCTEXT("SelectionFiltersLabel", "Selection Filters"));
								SelectionFilterSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FModelingToolsManagerCommands::Get().SelectionHitBackFaces, CommandList));
							}
							
							// Section for Geometry Isolation
							{
								FToolMenuSection& IsolationSection =
								Submenu->FindOrAddSection("Geometry Isolation", LOCTEXT("GeometryIsolationLabel", "Geometry Isolation"));
								IsolationSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().IsolateSelection, CommandList));
								IsolationSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().HideSelection, CommandList));
								IsolationSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().ShowFullMesh, CommandList));
							}

							// Section for Soft Selection
							{
								FToolMenuSection& SoftSelectionSection =
								Submenu->FindOrAddSection("Soft Selection", LOCTEXT("SoftSelectionLabel", "Soft Selection"));
								SoftSelectionSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FModelingToolsManagerCommands::Get().SelectionEnableSoftSelection, CommandList));
								const TSharedRef<SWidget> SoftSelectionRadiusWidget =
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.FillWidth(0.9f)
									[
										SNew(SSpinBox<double>)
										.MinValue(0.0f)
										.Value_Lambda(
											[WeakEdMode]() -> double 
											{
												const UGeometrySelectionManager* const SelectionManager = WeakEdMode.IsValid() ? WeakEdMode->GetSelectionManager() : nullptr;
												return SelectionManager ? SelectionManager->GetSoftSelectionRadius() : 0.0;
											})
										.OnValueChanged_Lambda(
											[WeakEdMode](double InValue)
												{
													UGeometrySelectionManager* const SelectionManager = WeakEdMode.IsValid() ? WeakEdMode->GetSelectionManager() : nullptr;
													if (SelectionManager)
													{
														SelectionManager->SetSoftSelectionRadius(InValue);
													}
												})
									]
									+ SHorizontalBox::Slot()
									.FillWidth(0.1f);
								
								SoftSelectionSection.AddEntry(
									FToolMenuEntry::InitWidget(
										"SoftSelectionRadius", 
										SoftSelectionRadiusWidget, 
										LOCTEXT("SoftSelectionRadiusLabel", "Radius")));

								PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
								auto GetSoftSelectionDistanceModeText = [](IGeometrySelector::ESoftSelectionDistanceMode InMode) -> FText
								{
									switch (InMode)
									{
									case IGeometrySelector::ESoftSelectionDistanceMode::Volume:
										return LOCTEXT("SoftSelectionDistanceModeVolume", "Volume");
									case IGeometrySelector::ESoftSelectionDistanceMode::Surface:
									default:
										return LOCTEXT("SoftSelectionDistanceModeSurface", "Surface");
									}
								};
								PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

								const TSharedRef<SWidget> SoftSelectionDistanceModeWidget =
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.FillWidth(0.9f)
									[
										SNew(SComboButton)
										.OnGetMenuContent_Lambda([WeakEdMode, GetSoftSelectionDistanceModeText]() -> TSharedRef<SWidget>
										{
											FMenuBuilder MenuBuilder(true, nullptr);

											PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
											auto AddModeEntry = [&MenuBuilder, WeakEdMode, GetSoftSelectionDistanceModeText](IGeometrySelector::ESoftSelectionDistanceMode InMode)
											{
												MenuBuilder.AddMenuEntry(
													GetSoftSelectionDistanceModeText(InMode),
													FText::GetEmpty(),
													FSlateIcon(),
													FUIAction(FExecuteAction::CreateLambda([WeakEdMode, InMode]()
													{
														UGeometrySelectionManager* const SelectionManager = WeakEdMode.IsValid() ? WeakEdMode->GetSelectionManager() : nullptr;
														if (SelectionManager)
														{
															SelectionManager->SetSoftSelectionDistanceMode(InMode);
														}
													})),
													NAME_None,
													EUserInterfaceActionType::None);
											};

											AddModeEntry(IGeometrySelector::ESoftSelectionDistanceMode::Surface);
											AddModeEntry(IGeometrySelector::ESoftSelectionDistanceMode::Volume);
											PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

											return MenuBuilder.MakeWidget();
										})
										.ButtonContent()
										[
											SNew(STextBlock)
											.Text_Lambda( [WeakEdMode, GetSoftSelectionDistanceModeText]() -> FText
											{
												PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
												const UGeometrySelectionManager* const SelectionManager = WeakEdMode.IsValid() ? WeakEdMode->GetSelectionManager() : nullptr;
												if (!SelectionManager)
												{
													return GetSoftSelectionDistanceModeText(IGeometrySelector::ESoftSelectionDistanceMode::Surface);
												}
												return GetSoftSelectionDistanceModeText(SelectionManager->GetSoftSelectionDistanceMode());
												PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
											})
										]
									]
									+ SHorizontalBox::Slot()
									.FillWidth(0.1f);

								SoftSelectionSection.AddEntry(
									FToolMenuEntry::InitWidget(
										"SoftSelectionDistanceMode",
										SoftSelectionDistanceModeWidget,
										LOCTEXT("SoftSelectionDistanceModeLabel", "Mode")));
							}

							// Section for Morph Target actions
							{
								FToolMenuSection& MorphTargetSection =
								Submenu->FindOrAddSection("Morph Target", LOCTEXT("MorphTargetLabel", "Morph Target"));
								MorphTargetSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().MirrorEditingMorphTarget, CommandList));
								MorphTargetSection.AddEntry(
									FToolMenuEntry::InitMenuEntryWithCommandList(
										FSkeletalMeshModelingToolsCommands::Get().FlipEditingMorphTarget, CommandList));
							}
						}
					})
			);

			Entry.ToolBarData.ResizeParams.ClippingPriority = 950;
			Entry.ToolBarData.LabelOverride = FText();
			Entry.InsertPosition = FToolMenuInsert("Transform", EToolMenuInsertType::After);
			Entry.Icon = FSlateIcon(FModelingToolsEditorModeStyle::GetStyleSetName(), "ModelingToolsManagerCommands.MeshElementSelection");

			FToolMenuSection* const LeftSection = InOutPersonaViewportToolbar.FindSection("Left");
			if (ensureMsgf(LeftSection, TEXT("Cannot find the left section in the persona viewport toolbar, cannot extend menu")))
			{
				LeftSection->AddDynamicEntry(TEXT("Mesh Element Selection"),
					FNewToolMenuSectionDelegate::CreateLambda([Entry](FToolMenuSection& InSection)
						{
							// Only extending the menu for skeletal mesh editor
							const USkeletalMeshToolMenuContext* Context = InSection.FindContext<USkeletalMeshToolMenuContext>();
							if (Context && Context->SkeletalMeshEditor.IsValid())
							{
								InSection.AddEntry(Entry);
							}
						})
				);
			}
		}
	} //~ namespace TransformMenuExtension

	static constexpr const TCHAR* SkeletalMeshEditorViewportToolbarOwnerName = TEXT("SkeletalMeshModelingToolsViewportToolbarExtension");

	void ExtendSkeletalMeshEditorViewportToolbar(
		USkeletalMeshModelingToolsEditorMode& EdMode, 
		const TSharedRef<FUICommandList>& CommandList)
	{
		const FToolMenuOwnerScoped ScopeOwner(SkeletalMeshEditorViewportToolbarOwnerName);

		UToolMenu* const PersonaViewportToolbar = UToolMenus::Get()->ExtendMenu("AnimationEditor.ViewportToolbar");
		if (ensureMsgf(PersonaViewportToolbar, TEXT("Cannot find the the persona viewport toolbar, cannot extend menus")))
		{
			SkeletalMeshEditorMenu::CreateSkeletalMeshEditorViewportToolbarMenu(CommandList, *PersonaViewportToolbar);
			TransformMenuExtension::ExtendTransformMenu(EdMode, CommandList, *PersonaViewportToolbar);
		}
	}

	void RemoveSkeletalMeshEditorViewportToolbarExtensions()
	{
		UToolMenus::Get()->UnregisterOwnerByName(SkeletalMeshEditorViewportToolbarOwnerName);
	}
}

#undef LOCTEXT_NAMESPACE
