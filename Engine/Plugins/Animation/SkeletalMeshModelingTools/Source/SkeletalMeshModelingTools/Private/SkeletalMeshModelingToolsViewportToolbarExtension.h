// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkeletalMeshModelingToolsEditorModeToolkit.h"

class FSkeletalMeshModelingToolsEditorModeToolkit;
class FUICommandList;

namespace UE::SkeletalMeshModelingTools
{
	void ExtendSkeletalMeshEditorViewportToolbar(
		USkeletalMeshModelingToolsEditorMode& EdMode,
		const TSharedRef<FUICommandList>& CommandList);

	void RemoveSkeletalMeshEditorViewportToolbarExtensions();
}
