// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/SKMBackedDynaMeshComponentToolTarget.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include "Components/SKMBackedDynaMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SKMBackedDynaMeshComponentToolTarget)

#define LOCTEXT_NAMESPACE "SKMBackedDynaMeshComponentToolTarget"


FReferenceSkeleton USkeletalMeshBackedDynamicMeshComponentToolTarget::GetSkeleton() const
{
	if (!GetComponent())
	{
		return {};
	}
	
	return GetComponent()->GetRefSkeleton();
}

void USkeletalMeshBackedDynamicMeshComponentToolTarget::CommitSkeletonModifier(USkeletonModifier* InModifier)
{
	if (!GetComponent())
	{
		return;
	}
	
	Super::CommitSkeletonModifier(InModifier);

	GetComponent()->HandleSkeletonChange(InModifier);
}

void USkeletalMeshBackedDynamicMeshComponentToolTarget::CommitDynamicMeshChange(TUniquePtr<FToolCommandChange> Change, const FText& ChangeMessage)
{
	if (!GetComponent())
	{
		return;
	}
	
	Super::CommitDynamicMeshChange(MoveTemp(Change), ChangeMessage);
	// we don't yet have a way of knowing if the tool changes the base geometry or not (either topo or just vert positions)
	// so we have to assume that it is dirty such that symmetry tracker is always up to date.
	GetComponent()->MarkBaseGeometryDirty();
	GetComponent()->MarkDirty();
}

USkeletalMesh* USkeletalMeshBackedDynamicMeshComponentToolTarget::GetSkeletalMesh() const
{
	if (!GetComponent())
	{
		return nullptr;
	}
	
	return GetComponent()->GetSkeletalMesh();
}

void USkeletalMeshBackedDynamicMeshComponentToolTarget::SetOwnerVisibility(bool bVisible) const
{
	if (!GetComponent())
	{
		return;
	}
	
	GetComponent()->ForwardVisibilityChangeRequest(bVisible);
}

USkeletalMeshBackedDynamicMeshComponent* USkeletalMeshBackedDynamicMeshComponentToolTarget::GetComponent() const
{
	if (!Component.IsValid())
	{
		return nullptr;
	}

	return CastChecked<USkeletalMeshBackedDynamicMeshComponent>(Component);
}

void USkeletalMeshBackedDynamicMeshComponentToolTargetFactory::Init(ISkeletalMeshBackedDynamicMeshComponentProvider* InComponentProvider)
{
	ComponentProvider = InComponentProvider;
}

bool USkeletalMeshBackedDynamicMeshComponentToolTargetFactory::CanBuildTarget(
	UObject* SourceObject,
	const FToolTargetTypeRequirements& Requirements
	) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMeshComponent, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of USkeletalMeshComponent,
	// just add another factory that allows that class specifically(but make sure that
	// GetMeshDescription and such work properly)
	
	bool bValid = Cast<USkeletalMeshComponent>(SourceObject) && Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset() &&
		ExactCast<USkeletalMesh>(Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset()) &&
		!ExactCast<USkeletalMesh>(Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset())->GetOutermost()->bIsCookedForEditor;
	if (!bValid)
	{
		return false;
	}

	if (!CanWriteToSource(SourceObject))
	{
		return false;
	}
	
	if (!ComponentProvider.IsValid())
	{
		return false;
	}

	USkeletalMeshBackedDynamicMeshComponent* Component = ComponentProvider->GetComponent(SourceObject);

	if (!Component)
	{
		return false;
	}

	return Super::CanBuildTarget(Component, Requirements);
}

UToolTarget* USkeletalMeshBackedDynamicMeshComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	if (!ComponentProvider.IsValid())
	{
		return nullptr;
	}

	USkeletalMeshBackedDynamicMeshComponent* Component = ComponentProvider->GetComponent(SourceObject);

	if (!Component)
	{
		return nullptr;
	}

	USkeletalMeshBackedDynamicMeshComponentToolTarget* Target = NewObject<USkeletalMeshBackedDynamicMeshComponentToolTarget>();
	Target->InitializeComponent(Cast<UDynamicMeshComponent>(Component));
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
	
}

bool USkeletalMeshBackedDynamicMeshComponentToolTargetFactory::CanWriteToSource(const UObject* SourceObject)
{
	if (const USkeletalMeshComponent* Component = GetValid(Cast<USkeletalMeshComponent>(SourceObject)))
	{
		if (const USkeletalMesh* SkeletalMesh = GetValid(Cast<USkeletalMesh>(Component->GetSkeletalMeshAsset())))
		{
			return !SkeletalMesh->GetPathName().StartsWith(TEXT("/Engine/"));
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
