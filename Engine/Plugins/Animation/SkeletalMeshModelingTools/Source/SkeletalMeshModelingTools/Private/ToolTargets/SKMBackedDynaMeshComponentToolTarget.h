// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/DynamicMeshComponent.h"
#include "Interfaces/ISKMBackedDynaMeshComponentProvider.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "UObject/WeakInterfacePtr.h"
#include "ToolTargets/ToolTarget.h"

#include "SKMBackedDynaMeshComponentToolTarget.generated.h"

#define UE_API SKELETALMESHMODELINGTOOLS_API

UCLASS(MinimalAPI, Transient)
class USkeletalMeshBackedDynamicMeshComponentToolTarget :
	public UDynamicMeshComponentToolTarget,
	public ISkeletalMeshBackedTarget
{
	GENERATED_BODY()
public:
	virtual FReferenceSkeleton GetSkeleton() const override;
	virtual void CommitSkeletonModifier(USkeletonModifier* InModifier) override;

	virtual void CommitDynamicMeshChange(TUniquePtr<FToolCommandChange> Change, const FText& ChangeMessage) override;
	
	virtual USkeletalMesh* GetSkeletalMesh() const override;

	virtual void SetOwnerVisibility(bool bVisible) const override;
protected:
	USkeletalMeshBackedDynamicMeshComponent* GetComponent() const;

	TWeakObjectPtr<USkeletalMesh> WeakSkeletalMesh;
	
	friend class USkeletalMeshBackedDynamicMeshComponentToolTargetFactory;
};

UCLASS(MinimalAPI, Transient)
class USkeletalMeshBackedDynamicMeshComponentToolTargetFactory : public UDynamicMeshComponentToolTargetFactory
{
	GENERATED_BODY()

public:
	void Init(ISkeletalMeshBackedDynamicMeshComponentProvider* InComponentProvider);

	virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;

	static bool CanWriteToSource(const UObject* SourceObject);

	TWeakInterfacePtr<ISkeletalMeshBackedDynamicMeshComponentProvider> ComponentProvider;
};

#undef UE_API
