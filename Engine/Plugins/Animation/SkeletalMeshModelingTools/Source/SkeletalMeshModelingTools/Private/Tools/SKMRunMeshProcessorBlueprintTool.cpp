// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/SKMRunMeshProcessorBlueprintTool.h"

#include "ContextObjectStore.h"
#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicSubmesh3.h"
#include "GeometryScript/EditorDynamicMeshProcessor.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "PreviewMesh.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "Components/DynamicMeshComponent.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/SceneComponentBackedTarget.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "ToolBuilderUtil.h"
#include "ToolTargetManager.h"
#include "ToolTargets/ToolTarget.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SKMRunMeshProcessorBlueprintTool)

#define LOCTEXT_NAMESPACE "SkeletalMeshRunMeshProcessorBlueprintTool"


// -----------------------------------------------------------------------------
// Properties
// -----------------------------------------------------------------------------

bool USkeletalMeshRunMeshProcessorBlueprintToolProperties::ShouldShowProcessorInstance() const
{
	if (!ProcessorBlueprintClass)
	{
		return false;
	}

	const UClass* GeneratedClass = ProcessorBlueprintClass.Get();
	if (!GeneratedClass)
	{
		return false;
	}

	for (TFieldIterator<FProperty> PropertyIt(GeneratedClass); PropertyIt; ++PropertyIt)
	{
		if (PropertyIt->HasAllPropertyFlags(CPF_Edit | CPF_BlueprintVisible))
		{
			return true;
		}
	}
	return false;
}


// -----------------------------------------------------------------------------
// Builder
// -----------------------------------------------------------------------------

const FToolTargetTypeRequirements& USkeletalMeshRunMeshProcessorBlueprintToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements ToolRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		USceneComponentBackedTarget::StaticClass(),
		USkeletonProvider::StaticClass(),
	});
	return ToolRequirements;
}

bool USkeletalMeshRunMeshProcessorBlueprintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// The tool reads pose / morph weights / isolation from the editor context object that the
	// SkeletalMeshModelingTools editor mode publishes at Enter() time. Without it the tool has
	// no skeletal data, so refuse to build.
	if (!SceneState.ToolManager->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>())
	{
		return false;
	}

	const int32 NumQualifyingTargets = SceneState.TargetManager->CountSelectedAndTargetable(
		SceneState, GetTargetRequirements());
	return NumQualifyingTargets == 1;
}

UInteractiveTool* USkeletalMeshRunMeshProcessorBlueprintToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	USkeletalMeshRunMeshProcessorBlueprintTool* Tool = NewObject<USkeletalMeshRunMeshProcessorBlueprintTool>(SceneState.ToolManager);
	Tool->SetTarget(SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements()));
	Tool->SetWorld(SceneState.World);
	return Tool;
}


// -----------------------------------------------------------------------------
// Tool
// -----------------------------------------------------------------------------

void USkeletalMeshRunMeshProcessorBlueprintTool::SetWorld(UWorld* InWorld)
{
	TargetWorld = InWorld;
}

void USkeletalMeshRunMeshProcessorBlueprintTool::Setup()
{
	Super::Setup();

	Settings = NewObject<USkeletalMeshRunMeshProcessorBlueprintToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	EditorContext = GetToolManager()
		->GetContextObjectStore()
		->FindContext<USkeletalMeshEditorContextObjectBase>();

	const FReferenceSkeleton& ReferenceSkeleton =
		CastChecked<ISkeletonProvider>(GetTarget())->GetSkeleton();
	ReferenceSkeleton.GetBoneAbsoluteTransforms(ComponentSpaceTransformsRefPose);

	static FGetMeshParameters MeshParams;
	MeshParams.bWantMeshTangents = true;
	FullSourceMesh = UE::ToolTarget::GetDynamicMeshCopy(GetTarget(), MeshParams);

	if (EditorContext.IsValid())
	{
		IsolatedTriangles = EditorContext->GetIsolatedTriangles();
	}

	RebuildProcessorInputMesh();

	UE::ToolTarget::HideSourceObject(GetTarget());

	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->CreateInWorld(TargetWorld.Get(),
		UE::ToolTarget::GetLocalToWorldTransform(GetTarget()));

	FComponentMaterialSet MaterialSet;
	Cast<IMaterialProvider>(GetTarget())->GetMaterialSet(MaterialSet);
	PreviewMesh->SetMaterials(MaterialSet.Materials);

	ToolDynamicMesh = NewObject<UDynamicMesh>(this);

	RebuildProcessorInstance(Settings->ProcessorBlueprintClass);
	RunProcessorAndUpdatePreview();

	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent()));

	PoseChangeDetector.GetNotifier().AddUObject(
		this, &USkeletalMeshRunMeshProcessorBlueprintTool::HandlePoseChangeDetectorEvent);

	if (EditorContext.IsValid())
	{
		EditorContext->ToggleBoneManipulation(true);
	}
}

void USkeletalMeshRunMeshProcessorBlueprintTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (Settings)
	{
		Settings->SaveProperties(this);
	}

	UnsubscribeFromBlueprintCompile();

	if (EditorContext.IsValid())
	{
		EditorContext->ToggleBoneManipulation(false);
	}

	// Commit the processed mesh back to the asset only when the user explicitly opted out of
	// preview mode and accepted. CanAccept() already gates the button on this condition.
	const bool bShouldCommit = ShutdownType == EToolShutdownType::Accept
		&& Settings && !Settings->bPreviewOnly
		&& ToolDynamicMesh
		&& AreAllTargetsValid();
	if (bShouldCommit)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("RunMeshProcBPTransaction", "Run Mesh Processor Blueprint"));
		ToolDynamicMesh->ProcessMesh([this](const UE::Geometry::FDynamicMesh3& InMesh)
		{
			constexpr bool bModifiedTopology = true;
			UE::ToolTarget::CommitDynamicMeshUpdate(GetTarget(), InMesh, bModifiedTopology);
		});
		GetToolManager()->EndUndoTransaction();
	}

	if (PreviewMesh)
	{
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

	if (AreAllTargetsValid())
	{
		UE::ToolTarget::ShowSourceObject(GetTarget());
	}

	Super::Shutdown(ShutdownType);
}

bool USkeletalMeshRunMeshProcessorBlueprintTool::CanAccept() const
{
	return Settings && !Settings->bPreviewOnly && AreAllTargetsValid();
}

void USkeletalMeshRunMeshProcessorBlueprintTool::OnTick(float DeltaTime)
{
	const TArray<FTransform>& ComponentSpaceTransforms = GetComponentSpaceBoneTransforms();
	const TMap<FName, float> MorphTargetWeights = GetMorphTargetWeights();

	PoseChangeDetector.CheckPose(ComponentSpaceTransforms, MorphTargetWeights);

	if (bFastDeformPreviewMesh)
	{
		bFastDeformPreviewMesh = false;
		PosePreviewMesh(ComponentSpaceTransforms, MorphTargetWeights);
	}
}

void USkeletalMeshRunMeshProcessorBlueprintTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	Super::OnPropertyModified(PropertySet, Property);

	if (PropertySet != Settings)
	{
		return;
	}

	const FName ChangedName = Property ? Property->GetFName() : NAME_None;

	if (ChangedName == GET_MEMBER_NAME_CHECKED(USkeletalMeshRunMeshProcessorBlueprintToolProperties, bPreviewOnly))
	{
		if (!Settings->bPreviewOnly)
		{
			GetToolManager()->DisplayMessage(
				LOCTEXT("RunOnSkeletalMeshWarning",
					"Running a mesh processor blueprint against a skeletal mesh is experimental, some geometry operations may not preserve all skeletal mesh attributes (skin weights, morph targets, etc.)."),
				EToolMessageLevel::UserWarning);
		}
		else
		{
			GetToolManager()->DisplayMessage(FText::GetEmpty(), EToolMessageLevel::UserWarning);
		}
		// Without isolation the BP already runs against the full mesh, so the preview matches
		// either mode and there's nothing to re-extract or re-run.
		if (!IsolatedTriangles.IsEmpty())
		{
			RebuildProcessorInputMesh();
			RunProcessorAndUpdatePreview();
		}
		return;
	}

	if (ChangedName == GET_MEMBER_NAME_CHECKED(USkeletalMeshRunMeshProcessorBlueprintToolProperties, ProcessorBlueprintClass))
	{
		RebuildProcessorInstance(Settings->ProcessorBlueprintClass);
	}

	RunProcessorAndUpdatePreview();
}

void USkeletalMeshRunMeshProcessorBlueprintTool::RebuildProcessorInputMesh()
{
	if (Settings && Settings->bPreviewOnly && !IsolatedTriangles.IsEmpty())
	{
		UE::Geometry::FDynamicSubmesh3 Submesh(&FullSourceMesh, IsolatedTriangles);
		ProcessorInputMesh = Submesh.GetSubmesh();
	}
	else
	{
		ProcessorInputMesh = FullSourceMesh;
	}
}

void USkeletalMeshRunMeshProcessorBlueprintTool::RebuildProcessorInstance(
	TSubclassOf<UEditorDynamicMeshProcessorBlueprint> NewClass)
{
	if (NewClass)
	{
		Settings->ProcessorInstance = NewObject<UDynamicMeshProcessorBlueprint>(
			Settings, NewClass, NAME_None, RF_Transactional);
	}
	else
	{
		Settings->ProcessorInstance = nullptr;
	}
	SubscribeToBlueprintCompile(NewClass.Get());
}

void USkeletalMeshRunMeshProcessorBlueprintTool::SubscribeToBlueprintCompile(UClass* GeneratedClass)
{
	UnsubscribeFromBlueprintCompile();
	if (!GeneratedClass)
	{
		return;
	}
	UBlueprint* Blueprint = Cast<UBlueprint>(GeneratedClass->ClassGeneratedBy);
	if (Blueprint)
	{
		CurrentProcessorBlueprint = Blueprint;
		BlueprintCompiledHandle = Blueprint->OnCompiled().AddUObject(
			this, &USkeletalMeshRunMeshProcessorBlueprintTool::HandleBlueprintCompiled);
	}
}

void USkeletalMeshRunMeshProcessorBlueprintTool::UnsubscribeFromBlueprintCompile()
{
	if (UBlueprint* Blueprint = CurrentProcessorBlueprint.Get())
	{
		Blueprint->OnCompiled().Remove(BlueprintCompiledHandle);
	}
	CurrentProcessorBlueprint = nullptr;
	BlueprintCompiledHandle.Reset();
}

void USkeletalMeshRunMeshProcessorBlueprintTool::HandleBlueprintCompiled(UBlueprint* InBlueprint)
{
	if (!InBlueprint || !Settings)
	{
		return;
	}
	// BP recompilation can replace the generated class. Refresh the property so subsequent
	// NewObject calls pick up the new class layout.
	Settings->ProcessorBlueprintClass = InBlueprint->GeneratedClass;
	RebuildProcessorInstance(Settings->ProcessorBlueprintClass);
	RunProcessorAndUpdatePreview();
}

void USkeletalMeshRunMeshProcessorBlueprintTool::RunProcessorAndUpdatePreview()
{
	ToolDynamicMesh->SetMesh(ProcessorInputMesh);

	if (Settings->ProcessorInstance)
	{
		bool bFailed = false;
		Settings->ProcessorInstance->ProcessDynamicMesh(ToolDynamicMesh, bFailed);
		if (bFailed)
		{
			GetToolManager()->DisplayMessage(
				LOCTEXT("ProcessorFailed", "Processor blueprint reported failure."),
				EToolMessageLevel::UserWarning);
		}
	}

	ToolDynamicMesh->ProcessMesh([this](const UE::Geometry::FDynamicMesh3& ProcessedMesh)
	{
		// Topology may have changed; rebuild render data fully.
		PreviewMesh->UpdatePreview(&ProcessedMesh, UPreviewMesh::ERenderUpdateMode::FullUpdate);
	});

	PosePreviewMesh(GetComponentSpaceBoneTransforms(), GetMorphTargetWeights());
}

void USkeletalMeshRunMeshProcessorBlueprintTool::PosePreviewMesh(
	const TArray<FTransform>& ComponentSpaceTransforms,
	const TMap<FName, float>& MorphTargetWeights)
{
	using namespace UE::Geometry;
	using namespace SkeletalMeshToolsHelper;

	if (ComponentSpaceTransforms.IsEmpty()
		|| ComponentSpaceTransforms.Num() != ComponentSpaceTransformsRefPose.Num())
	{
		return;
	}

	const TArray<FMatrix> BoneMatrices = ComputeBoneMatrices(
		ComponentSpaceTransformsRefPose, ComponentSpaceTransforms);

	ToolDynamicMesh->ProcessMesh([&](const FDynamicMesh3& Source)
	{
		constexpr bool bRebuildSpatial = false;
		PreviewMesh->DeferredEditMesh(
			[&](FDynamicMesh3& VisualMesh)
			{
				GetPosedMesh(
					[&VisualMesh](FVertInfo VertInfo, const FVector& PosedPosition)
					{
						VisualMesh.SetVertex(VertInfo.VertID, PosedPosition);
					},
					Source, BoneMatrices, NAME_None, MorphTargetWeights);

				FMeshNormals::QuickRecomputeOverlayNormals(VisualMesh);
			},
			bRebuildSpatial);

		const EMeshRenderAttributeFlags ChangedAttribs =
			EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals;
		PreviewMesh->NotifyDeferredEditCompleted(
			UPreviewMesh::ERenderUpdateMode::FastUpdate, ChangedAttribs, bRebuildSpatial);
	});
}

void USkeletalMeshRunMeshProcessorBlueprintTool::HandlePoseChangeDetectorEvent(
	SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload)
{
	using namespace SkeletalMeshToolsHelper;
	if (Payload.CurrentState == FPoseChangeDetector::PoseJustChanged ||
		Payload.CurrentState == FPoseChangeDetector::PoseChanged)
	{
		bFastDeformPreviewMesh = true;
	}
}

const TArray<FTransform>& USkeletalMeshRunMeshProcessorBlueprintTool::GetComponentSpaceBoneTransforms()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetComponentSpaceBoneTransforms(GetTarget());
	}

	if (DefaultComponentSpaceBoneTransforms.IsEmpty())
	{
		ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(GetTarget());
		SkeletonProvider->GetSkeleton().GetBoneAbsoluteTransforms(DefaultComponentSpaceBoneTransforms);
	}
	return DefaultComponentSpaceBoneTransforms;
}

TMap<FName, float> USkeletalMeshRunMeshProcessorBlueprintTool::GetMorphTargetWeights()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetMorphTargetWeights();
	}
	return {};
}

#undef LOCTEXT_NAMESPACE
