// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SingleSelectionTool.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"
#include "SkeletalMesh/ISkeletalMeshGeometryIsolationAwareTool.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryScript/EditorDynamicMeshProcessor.h"

#include "SKMRunMeshProcessorBlueprintTool.generated.h"

class UBlueprint;
class UDynamicMesh;
class UDynamicMeshProcessorBlueprint;
class UEditorDynamicMeshProcessorBlueprint;
class UPreviewMesh;
class USkeletalMeshEditorContextObjectBase;
class UWorld;


UCLASS()
class USkeletalMeshRunMeshProcessorBlueprintToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Mode", meta = (TransientToolProperty))
	bool bPreviewOnly = true;

	UPROPERTY(EditAnywhere, DisplayName = "Dyanmic Mesh Processor Blueprint", Category = "Processor")
	TSubclassOf<UEditorDynamicMeshProcessorBlueprint> ProcessorBlueprintClass;

	UPROPERTY(EditAnywhere, Instanced, NoClear, Category = "Processor",
		meta = (ShowOnlyInnerProperties, EditConditionHides, HideEditConditionToggle,
			EditCondition = "ShouldShowProcessorInstance"))
	TObjectPtr<UDynamicMeshProcessorBlueprint> ProcessorInstance;

	// Returns true only when the picked subclass exposes at least one BlueprintVisible editable
	// property — otherwise the instance section would render as an empty header. Inherited C++
	// UPROPERTYs that aren't BlueprintVisible (e.g. the base class's bRequiresGameThread) don't count.
	UFUNCTION()
	bool ShouldShowProcessorInstance() const;
};


UCLASS()
class USkeletalMeshRunMeshProcessorBlueprintToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	virtual const FToolTargetTypeRequirements& GetTargetRequirements() const;
};


/**
 * Run Mesh Processor Blueprint tool.
 */
UCLASS()
class USkeletalMeshRunMeshProcessorBlueprintTool :
	public USingleSelectionTool,
	public ISkeletalMeshGeometryIsolationAwareTool,
	public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void OnTick(float DeltaTime) override;
	virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;

	void SetWorld(UWorld* InWorld);

	virtual bool IsInputIsolationValidOnOutput() const override { return Settings && Settings->bPreviewOnly; }
	virtual bool IsInputSelectionValidOnOutput() override { return Settings && Settings->bPreviewOnly; }

protected:
	UPROPERTY()
	TObjectPtr<USkeletalMeshRunMeshProcessorBlueprintToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UDynamicMesh> ToolDynamicMesh;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshEditorContextObjectBase> EditorContext;

	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld;

	// Pristine full-mesh copy taken once at Setup. ProcessorInputMesh is rebuilt from this on
	// Setup and whenever bPreviewOnly toggles: in preview-only mode we extract the isolated
	// submesh (so the BP only operates on the visible region); when committing, we run the BP
	// against the full mesh because the result will be written back to the whole asset.
	UE::Geometry::FDynamicMesh3 FullSourceMesh;
	TArray<int32> IsolatedTriangles;
	UE::Geometry::FDynamicMesh3 ProcessorInputMesh;
	TArray<FTransform> ComponentSpaceTransformsRefPose;

	SkeletalMeshToolsHelper::FPoseChangeDetector PoseChangeDetector;
	bool bFastDeformPreviewMesh = false;

	void RebuildProcessorInstance(TSubclassOf<UEditorDynamicMeshProcessorBlueprint> NewClass);
	void RebuildProcessorInputMesh();
	void RunProcessorAndUpdatePreview();
	void PosePreviewMesh(
		const TArray<FTransform>& ComponentSpaceTransforms,
		const TMap<FName, float>& MorphTargetWeights);

	void HandlePoseChangeDetectorEvent(SkeletalMeshToolsHelper::FPoseChangeDetector::FPayload Payload);

	// Refresh the instance and re-run the processor when the BP author recompiles their graph.
	void SubscribeToBlueprintCompile(UClass* GeneratedClass);
	void UnsubscribeFromBlueprintCompile();
	void HandleBlueprintCompiled(UBlueprint* InBlueprint);

	TWeakObjectPtr<UBlueprint> CurrentProcessorBlueprint;
	FDelegateHandle BlueprintCompiledHandle;

	const TArray<FTransform>& GetComponentSpaceBoneTransforms();
	TMap<FName, float> GetMorphTargetWeights();

	// Fallback when EditorContext is unavailable.
	TArray<FTransform> DefaultComponentSpaceBoneTransforms;
};
