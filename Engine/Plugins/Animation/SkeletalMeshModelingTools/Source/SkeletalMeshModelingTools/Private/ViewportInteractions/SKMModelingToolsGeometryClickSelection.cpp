// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/SKMModelingToolsGeometryClickSelection.h"

#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "Selection/GeometrySelectionManager.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

USkeletalMeshModelingToolsGeometryClickSelection::USkeletalMeshModelingToolsGeometryClickSelection()
{
	InteractionName = TEXT("Skeletal Mesh Modeling Tools Geometry Click Selection");
	Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void USkeletalMeshModelingToolsGeometryClickSelection::BuildBehaviors()
{
	Super::BuildBehaviors();
	
	if (UViewportClickBehavior* ClickBehavior = ClickBehaviorWeak.Get())
	{
		using namespace UE::Editor::ViewportInteractions;
	
		ClickBehavior->SetBindings({
			FButtonBinding(EKeys::LeftMouseButton).TriggersStart(true),
			FButtonBinding(EKeys::LeftControl).Required(false),
			FButtonBinding(EKeys::LeftShift).Required(false)
		});
	}
}

void USkeletalMeshModelingToolsGeometryClickSelection::OnClickUp(const FInputDeviceRay& InClickPos)
{
	if (UViewportInteractionsBehaviorSource* const ViewportInteractionBehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		ViewportInteractionBehaviorSource->ClearMouseCursorOverride();
	}

	if (IsMouseLooking())
	{
		if (UViewportInteractionsBehaviorSource* const ViewportInteractionBehaviorSource = GetViewportInteractionsBehaviorSource())
		{
			// Store info about camera movement locally, since SetIsMouseLooking call will reset it
			const bool bCameraHasMoved = ViewportInteractionBehaviorSource->HasCameraMoved();
			ViewportInteractionBehaviorSource->SetIsMouseLooking(false);

			// If camera has moved, this was definitely not a click
			if (bCameraHasMoved)
			{
				return;
			}
		}
	}

	UE::Geometry::FGeometrySelectionUpdateConfig UpdateConfig;
	UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Replace;
	if (IsShiftDown())
	{
		UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Add;
	}
	else if (IsCtrlDown())
	{
		UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Remove;
	}
	UE::Geometry::FGeometrySelectionUpdateResult Result;
	SelectionManager->UpdateSelectionViaRaycast(InClickPos.WorldRay, UpdateConfig, Result);	
}

void USkeletalMeshModelingToolsGeometryClickSelection::BindSelectionManager(UGeometrySelectionManager* InSelectionManager)
{
	SelectionManager = InSelectionManager;
}
