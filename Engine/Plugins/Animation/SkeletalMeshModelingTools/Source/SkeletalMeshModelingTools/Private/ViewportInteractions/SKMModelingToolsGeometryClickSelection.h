// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportInteractions/ViewportClickInteraction.h"
#include "SKMModelingToolsGeometryClickSelection.generated.h"

class UGeometrySelectionManager;

UCLASS(MinimalAPI, Transient)
class USkeletalMeshModelingToolsGeometryClickSelection
	: public UViewportClickInteraction
{
	GENERATED_BODY()

public:
	USkeletalMeshModelingToolsGeometryClickSelection();
	virtual void BuildBehaviors() override;
	virtual void OnClickUp(const FInputDeviceRay& InClickPos) override;

	void BindSelectionManager(UGeometrySelectionManager* InSelectionManager);

	virtual TArray<FEditorModeID> GetUnsupportedModes() const override {return {};}
	
protected:
	
	TWeakObjectPtr<UGeometrySelectionManager> SelectionManager;
};

