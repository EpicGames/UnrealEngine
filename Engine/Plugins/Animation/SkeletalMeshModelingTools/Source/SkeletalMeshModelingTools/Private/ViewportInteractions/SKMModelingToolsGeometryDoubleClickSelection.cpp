// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/SKMModelingToolsGeometryDoubleClickSelection.h"

#include "Commands/ModifyGeometrySelectionCommand.h"
#include "Components/PrimitiveComponent.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Selection/GeometrySelectionManager.h"
#include "Selections/GeometrySelection.h"
#include "ToolContextInterfaces.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"

#define LOCTEXT_NAMESPACE "SKMModelingToolsGeometryDoubleClickSelection"

TOptional<FInputCapturePriority> USkeletalMeshModelingToolsGeometryDoubleClickBehavior::GetCapturePriority(
	UE::Editor::ViewportInteractions::EInputStage Stage,
	const FInputDeviceState& InputState) const
{
	if (InputState.Mouse.Left.bDoubleClicked)
	{
		return FInputCapturePriority(FInputCapturePriority::DEFAULT_GIZMO_PRIORITY - 1);
	}
	return TOptional<FInputCapturePriority>();
}


USkeletalMeshModelingToolsGeometryDoubleClickSelection::USkeletalMeshModelingToolsGeometryDoubleClickSelection()
{
	InteractionName = TEXT("Skeletal Mesh Modeling Tools Geometry Double Click Selection");
	Groups = { UE::Editor::ViewportInteractions::ViewportClick };
}

void USkeletalMeshModelingToolsGeometryDoubleClickSelection::BuildBehaviors()
{
	using namespace UE::Editor::ViewportInteractions;

	USkeletalMeshModelingToolsGeometryDoubleClickBehavior* ClickBehavior =
		NewObject<USkeletalMeshModelingToolsGeometryDoubleClickBehavior>();
	ClickBehavior->Initialize(this);
	ClickBehavior->SetAllowsCaptureStealing(true);
	ClickBehavior->SetBindings({
		FButtonBinding(EKeys::LeftMouseButton).TriggersStart(true),
		FButtonBinding(EKeys::LeftControl).Required(false),
		FButtonBinding(EKeys::LeftShift).Required(false)
	});

	RegisterInputBehavior(ClickBehavior);
	ClickBehaviorWeak = ClickBehavior;
}

void USkeletalMeshModelingToolsGeometryDoubleClickSelection::OnClickUp(const FInputDeviceRay& InClickPos)
{
	if (UViewportInteractionsBehaviorSource* const ViewportInteractionBehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		ViewportInteractionBehaviorSource->ClearMouseCursorOverride();
	}

	if (!SelectionManager.IsValid())
	{
		return;
	}

	const bool bShift = IsShiftDown();
	const bool bCtrl  = IsCtrlDown();

	IToolsContextTransactionsAPI* TransactionsAPI = SelectionManager->GetTransactionsAPI();

	if (bCtrl)
	{
		UPrimitiveComponent* const EditingComponent = GetEditingComponent ? GetEditingComponent() : nullptr;
		if (!EditingComponent)
		{
			return;
		}

		UE::Geometry::FGeometrySelection PriorSelection;
		SelectionManager->GetSelectionForComponent(EditingComponent, PriorSelection);

		if (TransactionsAPI)
		{
			TransactionsAPI->BeginUndoTransaction(LOCTEXT("RemoveConnectedIslandTransaction", "Remove Connected Island"));
		}

		UE::Geometry::FGeometrySelectionUpdateConfig ReplaceConfig;
		ReplaceConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Replace;
		UE::Geometry::FGeometrySelectionUpdateResult Result;
		SelectionManager->UpdateSelectionViaRaycast(InClickPos.WorldRay, ReplaceConfig, Result);

		UModifyGeometrySelectionCommand_ExpandToConnected* ExpandCommand =
			NewObject<UModifyGeometrySelectionCommand_ExpandToConnected>(SelectionManager.Get());
		SelectionManager->ExecuteSelectionCommand(ExpandCommand);

		UE::Geometry::FGeometrySelection IslandSelection;
		SelectionManager->GetSelectionForComponent(EditingComponent, IslandSelection);

		UE::Geometry::FGeometrySelection FinalSelection = PriorSelection;
		for (uint64 IslandElement : IslandSelection.Selection)
		{
			FinalSelection.Selection.Remove(IslandElement);
		}
		SelectionManager->SetSelectionForComponent(EditingComponent, FinalSelection);

		if (TransactionsAPI)
		{
			TransactionsAPI->EndUndoTransaction();
		}
		return;
	}

	UE::Geometry::FGeometrySelectionUpdateConfig UpdateConfig;
	UpdateConfig.ChangeType = bShift
		? UE::Geometry::EGeometrySelectionChangeType::Add
		: UE::Geometry::EGeometrySelectionChangeType::Replace;

	if (TransactionsAPI)
	{
		TransactionsAPI->BeginUndoTransaction(
			bShift
				? LOCTEXT("AddConnectedIslandTransaction", "Add Connected Island")
				: LOCTEXT("SelectConnectedIslandTransaction", "Select Connected Island"));
	}

	UE::Geometry::FGeometrySelectionUpdateResult Result;
	SelectionManager->UpdateSelectionViaRaycast(InClickPos.WorldRay, UpdateConfig, Result);

	UModifyGeometrySelectionCommand_ExpandToConnected* ExpandCommand =
		NewObject<UModifyGeometrySelectionCommand_ExpandToConnected>(SelectionManager.Get());
	SelectionManager->ExecuteSelectionCommand(ExpandCommand);

	if (TransactionsAPI)
	{
		TransactionsAPI->EndUndoTransaction();
	}
}

void USkeletalMeshModelingToolsGeometryDoubleClickSelection::BindSelection(
	UGeometrySelectionManager* InSelectionManager,
	TFunction<UPrimitiveComponent*()> InGetEditingComponent)
{
	SelectionManager = InSelectionManager;
	GetEditingComponent = MoveTemp(InGetEditingComponent);
}

#undef LOCTEXT_NAMESPACE
