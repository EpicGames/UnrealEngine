// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ViewportClickBehavior.h"
#include "ViewportInteractions/ViewportClickInteraction.h"
#include "SKMModelingToolsGeometryDoubleClickSelection.generated.h"

class UGeometrySelectionManager;
class UPrimitiveComponent;

UCLASS(MinimalAPI, Transient)
class USkeletalMeshModelingToolsGeometryDoubleClickBehavior : public UViewportClickBehavior
{
	GENERATED_BODY()

protected:
	virtual TOptional<FInputCapturePriority> GetCapturePriority(
		UE::Editor::ViewportInteractions::EInputStage Stage,
		const FInputDeviceState& InputState) const override;
};

UCLASS(MinimalAPI, Transient)
class USkeletalMeshModelingToolsGeometryDoubleClickSelection
	: public UViewportClickInteraction
{
	GENERATED_BODY()

public:
	USkeletalMeshModelingToolsGeometryDoubleClickSelection();
	virtual void BuildBehaviors() override;
	virtual void OnClickUp(const FInputDeviceRay& InClickPos) override;

	void BindSelection(
		UGeometrySelectionManager* InSelectionManager,
		TFunction<UPrimitiveComponent*()> InGetEditingComponent);

	virtual TArray<FEditorModeID> GetUnsupportedModes() const override { return {}; }

protected:
	TWeakObjectPtr<UGeometrySelectionManager> SelectionManager;
	TFunction<UPrimitiveComponent*()> GetEditingComponent;
};
