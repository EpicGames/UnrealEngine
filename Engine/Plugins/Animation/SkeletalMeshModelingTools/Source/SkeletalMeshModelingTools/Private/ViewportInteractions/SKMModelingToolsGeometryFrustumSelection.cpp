// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/SKMModelingToolsGeometryFrustumSelection.h"

#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "Selection/GeometrySelectionManager.h"

USkeletalMeshModelingToolsGeometryFrustumSelection::USkeletalMeshModelingToolsGeometryFrustumSelection()
{
	InteractionName = TEXT("Skeletal Mesh Modeling Tools Geometry Frustum Selection");
	Groups = { UE::Editor::ViewportInteractions::FrustumSelect };

	if (UViewportClickDragBehavior* ClickDragBehavior = ClickDragInputBehavior.Get())
	{
		using namespace UE::Editor::ViewportInteractions;
		ClickDragBehavior->SetBindings({
			FButtonBinding(EKeys::LeftMouseButton).TriggersStart(),
			FButtonBinding(EKeys::LeftControl).Required(false),
			FButtonBinding(EKeys::LeftShift).Required(false)
		});
	}
}

void USkeletalMeshModelingToolsGeometryFrustumSelection::BindSelectionManager(UGeometrySelectionManager* InSelectionManager)
{
	SelectionManager = InSelectionManager;
}

FInputRayHit USkeletalMeshModelingToolsGeometryFrustumSelection::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (IsEnabled())
	{
		return Super::CanBeginClickDragSequence(PressPos);
	}

	return FInputRayHit(); 
}

void USkeletalMeshModelingToolsGeometryFrustumSelection::OnDragEnd(const FInputDeviceRay& InReleasePos)
{
	End = FVector(InReleasePos.ScreenPosition.X, InReleasePos.ScreenPosition.Y, 0);

	if (!EditorViewportClientProxy)
	{
		return;
	}

	FEditorViewportClient* const EditorViewportClient = GetEditorViewportClient();
	if (!EditorViewportClient)
	{
		return;
	}

	FViewport* Viewport = EditorViewportClient->Viewport;
	if (!Viewport)
	{
		return;
	}

	FEditorModeTools* ModeTools = EditorViewportClient->GetModeTools();
	if (!ModeTools)
	{
		return;
	}

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags)
	);
	FSceneView* SceneView = EditorViewportClient->CalcSceneView(&ViewFamily);

	// Generate a frustum out of the dragged box
	FConvexVolume Frustum;
	constexpr bool bUseBoxFrustum = true;
	UE::EditorDragTools::FViewportFrustum ViewportFrustum(*EditorViewportClient, *SceneView, Start, End, bUseBoxFrustum);
	ViewportFrustum.Calculate(Frustum);

	if (SelectionManager.IsValid()
		&& SelectionManager->HasActiveTargets() 
		&& SelectionManager->GetMeshTopologyMode() != UGeometrySelectionManager::EMeshTopologyMode::None)
	{
		UE::Geometry::FGeometrySelectionUpdateConfig UpdateConfig;
		UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Replace;
		if (FInputDeviceState::IsShiftKeyDown(InputState))
		{
			UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Add;
		}
		else if ( FInputDeviceState::IsCtrlKeyDown(InputState))
		{
			UpdateConfig.ChangeType = UE::Geometry::EGeometrySelectionChangeType::Remove;
		}			

		UE::Geometry::FGeometrySelectionUpdateResult Result;
		SelectionManager->UpdateSelectionViaConvex(
			Frustum, UpdateConfig, Result);
	}

	UDragToolInteraction::OnDragEnd(InReleasePos);
}
