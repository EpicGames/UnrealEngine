// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorDragTools/FrustumSelectInteraction.h"
#include "SKMModelingToolsGeometryFrustumSelection.generated.h"

class UGeometrySelectionManager;

UCLASS(MinimalAPI, Transient)
class USkeletalMeshModelingToolsGeometryFrustumSelection : public UFrustumSelectInteraction
{
	GENERATED_BODY()

public:
	USkeletalMeshModelingToolsGeometryFrustumSelection();

	void BindSelectionManager(UGeometrySelectionManager* InSelectionManager);

	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnDragEnd(const FInputDeviceRay& InReleasePos) override;

	virtual TArray<FEditorModeID> GetUnsupportedModes() const override {return {};}
	
protected:
	TWeakObjectPtr<UGeometrySelectionManager> SelectionManager;
};

