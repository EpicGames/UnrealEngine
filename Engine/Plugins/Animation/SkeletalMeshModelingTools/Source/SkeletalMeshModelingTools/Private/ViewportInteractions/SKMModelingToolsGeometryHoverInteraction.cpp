// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/SKMModelingToolsGeometryHoverInteraction.h"

#include "BaseBehaviors/MouseHoverBehavior.h"
#include "Selection/GeometrySelectionManager.h"


USkeletalMeshModelingToolsGeometryHoverInteraction::USkeletalMeshModelingToolsGeometryHoverInteraction()
{
	InteractionName = TEXT("Skeletal Mesh Modeling Tools Geometry Hover Interaction");
	
	HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	
	RegisterInputBehavior(HoverBehavior);
}

void USkeletalMeshModelingToolsGeometryHoverInteraction::BindSelectionManager(UGeometrySelectionManager* InSelectionManager)
{
	SelectionManager = InSelectionManager;
}

FInputRayHit USkeletalMeshModelingToolsGeometryHoverInteraction::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	if (!IsEnabled())
	{
		return {};
	}

	FInputRayHit ActiveObjectHit = FInputRayHit();
	SelectionManager->RayHitTest(PressPos.WorldRay, ActiveObjectHit);
	return ActiveObjectHit;
}


void USkeletalMeshModelingToolsGeometryHoverInteraction::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	SelectionManager->UpdateSelectionPreviewViaRaycast(DevicePos.WorldRay);
}

bool USkeletalMeshModelingToolsGeometryHoverInteraction::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	return SelectionManager->UpdateSelectionPreviewViaRaycast(DevicePos.WorldRay);
}

void USkeletalMeshModelingToolsGeometryHoverInteraction::OnEndHover()
{
	SelectionManager->ClearSelectionPreview();
}

