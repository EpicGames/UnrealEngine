// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkeletalMeshModelingToolsEditorMode.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ViewportInteractions/ViewportInteraction.h"
#include "SKMModelingToolsGeometryHoverInteraction.generated.h"



UCLASS(MinimalAPI, Transient)
class USkeletalMeshModelingToolsGeometryHoverInteraction
	: public UViewportInteraction
	, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	USkeletalMeshModelingToolsGeometryHoverInteraction();

	void BindSelectionManager(UGeometrySelectionManager* InSelectionManager);
	
	/**
	 * Do hover hit-test
	 */
	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;

	/**
	 * Initialize hover sequence at given position
	 */
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;

	/**
	 * Update active hover sequence with new input position
	 */
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	/**
	 * Terminate active hover sequence
	 */
	virtual void OnEndHover() override;
	
protected:	
	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior;

	TWeakObjectPtr<UGeometrySelectionManager> SelectionManager;
	
};

