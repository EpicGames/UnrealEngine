// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportInteractions/SKMModelingToolsGeometrySoftSelectRadiusDrag.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EditorViewportClient.h"
#include "MeshAdapterTransforms.h"
#include "SceneView.h"
#include "EditorDragTools/EditorViewportClientProxy.h"
#include "Selection/GeometrySelectionManager.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"


USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction()
{
	InteractionName = TEXT("Skeletal Mesh Modeling Tools Geometry Soft Select Radius Drag Interaction");
	
	UViewportClickDragBehavior* MouseBehavior = NewObject<UViewportClickDragBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->SetBindings({
		UE::Editor::ViewportInteractions::FButtonBinding(EKeys::LeftMouseButton).TriggersStart(),
		UE::Editor::ViewportInteractions::FButtonBinding(EKeys::B),
	});

	RegisterInputBehavior(MouseBehavior);
}

void USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::BindSelectionManager(UGeometrySelectionManager* InSelectionManager)
{
	SelectionManager = InSelectionManager;
}

void USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	if (!ensure(InRenderAPI) || !ensure(InCanvas))
	{
		return;
	}

	if (!IsDragging() || !EnumHasAnyFlags(InRenderAPI->GetViewInteractionState(), EViewInteractionState::Focused))
	{
		return;
	}

	if (!SelectionManager->HasSelection())
	{
		return;
	}
	
	UE::Geometry::FFrame3d SelectionFrame; 
	SelectionManager->GetSelectionWorldFrame(SelectionFrame);

	double Radius = SelectionManager->GetSoftSelectionRadius();

	const FSceneView* SceneView = InRenderAPI->GetSceneView();
	if (!SceneView)
	{
		return;
	}
	
	FVector2D OriginPixel; 
	SceneView->WorldToPixel(SelectionFrame.Origin, OriginPixel);
	OriginPixel /= InCanvas->GetDPIScale();

	FVector RadiusSamplePoint = SelectionFrame.Origin + SceneView->GetViewUp() * Radius;
	FVector2D RadiusSamplePointPixel;
	SceneView->WorldToPixel(RadiusSamplePoint, RadiusSamplePointPixel);
	RadiusSamplePointPixel /= InCanvas->GetDPIScale();

	double PixelRadius = FMath::Abs(RadiusSamplePointPixel.Y - OriginPixel.Y);

	const int32 NumSides = FMath::Max(64, (int)PixelRadius / 25);
	const double AngleDelta = 2.0 * UE_PI / NumSides;
	const FVector2D AxisX(1., 0.);
	const FVector2D AxisY(0., -1.);
	FVector2D LastVertex = OriginPixel + AxisX * PixelRadius;

	for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
	{
		const double CurAngle = AngleDelta * (SideIndex + 1);
		const FVector2D Vertex = OriginPixel + (AxisX * FMath::Cos(CurAngle) + AxisY * FMath::Sin(CurAngle)) * PixelRadius;

		FCanvasLineItem LineItem{ FVector2D(LastVertex), FVector2D(Vertex) };
		LineItem.SetColor(FLinearColor::Green);
		InCanvas->DrawItem(LineItem);

		LastVertex = Vertex;
	}

}

FInputRayHit USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (IsEnabled())
	{
		if (SelectionManager->HasSelection())
		{
			return FInputRayHit(TNumericLimits<float>::Max()); // bHit is true.
		}
	}

	return FInputRayHit();
}

void USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::OnDragStart(const FInputDeviceRay& InPressPos)
{
	if (UViewportInteractionsBehaviorSource* BehaviorSource = GetViewportInteractionsBehaviorSource())
	{
		BehaviorSource->SetMouseCursorOverride(EMouseCursor::Default);
	}

	Start = FVector(InPressPos.ScreenPosition.X, InPressPos.ScreenPosition.Y, 0);
	End = Start;
	
	SelectionManager->SetSoftSelectionEnabled(true);
	StartingRadius = SelectionManager->GetSoftSelectionRadius();

	bIsDragging = true;
	
	// Signal that this tool is now active
	OnActivateTool().Broadcast();

}

void USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::OnDrag(const FDragArgs& InDrag)
{
	End = FVector(InDrag.Ray.ScreenPosition.X, InDrag.Ray.ScreenPosition.Y, 0);
	static double Multiplier = 1;
	FEditorViewportClient* EditorViewportClient = GetEditorViewportClient();

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(EditorViewportClient->Viewport, EditorViewportClient->GetScene(), EditorViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(EditorViewportClient->IsRealtime()));

	FSceneView* View = EditorViewportClient->CalcSceneView(&ViewFamily);

	UE::Geometry::FFrame3d SelectionFrame; 
	SelectionManager->GetSelectionWorldFrame(SelectionFrame);

	double SampleRadius = 10;
	
	FVector2D OriginPixel; 
	View->WorldToPixel(SelectionFrame.Origin, OriginPixel);

	FVector RadiusSamplePoint = SelectionFrame.Origin + View->GetViewUp() * SampleRadius;
	FVector2D RadiusSamplePointPixel;
	View->WorldToPixel(RadiusSamplePoint, RadiusSamplePointPixel);
	
	const double PixelRadius = FMath::Abs(RadiusSamplePointPixel.Y - OriginPixel.Y);

	double CentimeterPerPixel = SampleRadius / PixelRadius;
	
	SelectionManager->SetSoftSelectionRadius(StartingRadius + (End.X - Start.X) * CentimeterPerPixel );
}

void USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction::OnDragEnd(const FInputDeviceRay& InReleasePos)
{
	if (SelectionManager->GetSoftSelectionRadius() == 0)
	{
		SelectionManager->SetSoftSelectionEnabled(false);
	}
	
	Super::OnDragEnd(InReleasePos);
}
