// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorDragTools/DragToolInteraction.h"

#include "SKMModelingToolsGeometrySoftSelectRadiusDrag.generated.h"

class UGeometrySelectionManager;

UCLASS(MinimalAPI, Transient)
class USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction
	: public UDragToolInteraction
{
	GENERATED_BODY()

public:
	USkeletalMeshModelingToolsGeometrySoftSelectRadiusDragInteraction();

	void BindSelectionManager(UGeometrySelectionManager* InSelectionManager);

	virtual void Draw(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;

	//~ Begin IViewportClickDragBehaviorTarget
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	virtual void OnDragStart(const FInputDeviceRay& InPressPos) override;
	virtual void OnDrag(const FDragArgs& InDrag) override;
	virtual void OnDragEnd(const FInputDeviceRay& InReleasePos) override;
	//~ End IViewportClickDragBehaviorTarget

protected:
	virtual TArray<FEditorModeID> GetUnsupportedModes() const override { return {}; }
	
protected:
	double StartingRadius = 0;
	
	TWeakObjectPtr<UGeometrySelectionManager> SelectionManager;
};
