// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyHandle.h"
#include "SEnumCombo.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;

namespace UE::SkeletalMeshModelingTools
{
	/** A combo box for enum values. Useful to exclude elements */
	template <typename TEnumType>
	class SSKMEnumPropertyComboBox
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSKMEnumPropertyComboBox<TEnumType>)
			{}

			/** The visibility attribute for this widget */
			SLATE_ATTRIBUTE(EVisibility, Visibility);

			/** The enabled attribute for this widget. Only considered when the enum property handle is editable */
			SLATE_ATTRIBUTE(bool, IsEnabled);

			/** Enumerators that should be excluded from the combo box */
			SLATE_ARGUMENT(TSet<TEnumType>, ExcludedEnumerators);
		
		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InEnumPropertyHandle)
		{
			const UEnum* Enum = StaticEnum<TEnumType>();
			if (!ensureMsgf(Enum, TEXT("Invalid enum type when trying to show an SSKMEnumComboBox in the SKM editor. Cannot display enum.")))
			{
				return;
			}			

			UnderlyingType = [&InEnumPropertyHandle, this]() -> const FFieldClass*
				{
					if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InEnumPropertyHandle->GetProperty()))
					{
						const FNumericProperty* UnderlyingTypeProperty = EnumProperty ? EnumProperty->GetUnderlyingProperty() : nullptr;
						return UnderlyingTypeProperty ? UnderlyingTypeProperty->GetClass() : nullptr;
					}
					else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InEnumPropertyHandle->GetProperty()))
					{
						return ByteProperty->GetClass();
					}
					return nullptr;
				}();
			if (!ensureMsgf(UnderlyingType, TEXT("Unsupported enum type when trying to show an SSKMEnumComboBox in the SKM editor. Cannot display enum.")))
			{
				return;
			}

			EnumPropertyHandle = InEnumPropertyHandle;
			IsEnabledAttribute = InArgs._IsEnabled;

			TArray<int32> AvaialableValues;
			for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; EnumIndex++)
			{
				const int32 EnumValue = Enum->GetValueByIndex(EnumIndex);
				if (!InArgs._ExcludedEnumerators.Contains(static_cast<TEnumType>(EnumValue)))
				{
					const int32 Value = static_cast<int32>(Enum->GetValueByIndex(EnumIndex));
					AvaialableValues.Add(Value);
				}
			}
			
			if (AvaialableValues.IsEmpty())
			{
				return;
			}

			if (!AvaialableValues.Contains(GetValue()))
			{
				// Set X for the value if none is hidden and selected
				SetValue(AvaialableValues[0], ESelectInfo::Direct);
			}

			ChildSlot
			[
				SNew(SEnumComboBox, StaticEnum<TEnumType>())
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))		
				.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
				.Visibility(InArgs._Visibility)
				.IsEnabled(this, &SSKMEnumPropertyComboBox::IsEnabled)
				.CurrentValue(this, &SSKMEnumPropertyComboBox::GetValue)
				.OnEnumSelectionChanged(this, &SSKMEnumPropertyComboBox::SetValue)
				.EnumValueSubset(AvaialableValues)
			];
		}

	private:
		/** Returns true if the widget is enabled */
		bool IsEnabled() const
		{
			if (!EnumPropertyHandle.IsValid() ||
				!EnumPropertyHandle->IsEditable())
			{
				return false;
			}

			if (IsEnabledAttribute.IsSet())
			{
				return IsEnabledAttribute.Get();
			}

			return true;
		}

		/** Returns the currently selected axis */
		int32 GetValue() const
		{
			if (EnumPropertyHandle.IsValid() &&
				EnumPropertyHandle->IsValidHandle())
			{
				if (UnderlyingType == FByteProperty::StaticClass())
				{
					uint8 Value;
					if (EnumPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
					{
						return Value;
					}
				}
				else if (UnderlyingType == FIntProperty::StaticClass())
				{
					int32 Value;
					if (EnumPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
					{
						return Value;
					}
				}
				else
				{
					ensureMsgf(0, TEXT("Unsupported type"));
				}
			}

			return 0;
		}

		/** Called when an axis was selected */
		void SetValue(int32 EnumValue, ESelectInfo::Type SelectInfo)
		{
			if (EnumPropertyHandle.IsValid() &&
				EnumPropertyHandle->IsValidHandle())
			{
				EnumPropertyHandle->NotifyPreChange();

				if (UnderlyingType == FByteProperty::StaticClass())
				{
					EnumPropertyHandle->SetValue(static_cast<uint8>(EnumValue));
				}
				else if (UnderlyingType == FIntProperty::StaticClass())
				{
					EnumPropertyHandle->SetValue(EnumValue);
				}
				else
				{
					ensureMsgf(0, TEXT("Unsupported type"));
				}

				EnumPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			}
		}

		/** The size type of the enum */
		const FFieldClass* UnderlyingType = nullptr;

		/** The enabled attribute for this widget. Only considered when the enum property handle is editable. */
		TAttribute<bool> IsEnabledAttribute;

		/** The property handle to the enum */
		TSharedPtr<IPropertyHandle> EnumPropertyHandle;
	};
}
