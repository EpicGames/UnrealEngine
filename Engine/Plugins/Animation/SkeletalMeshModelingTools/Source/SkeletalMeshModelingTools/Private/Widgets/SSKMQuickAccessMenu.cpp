// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSKMQuickAccessMenu.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

void SSkeletalMeshModelingToolsQuickAccessMenu::Construct(const FArguments& InArgs)
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection=*/true, InArgs._CommandList);
	InArgs._OnPopulateMenu.ExecuteIfBound(MenuBuilder);

	ChildSlot
	[
		MenuBuilder.MakeWidget()
	];
}
