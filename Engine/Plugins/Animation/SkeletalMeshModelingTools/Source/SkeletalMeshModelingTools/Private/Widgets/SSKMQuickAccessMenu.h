// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FMenuBuilder;
class FUICommandList;

DECLARE_DELEGATE_OneParam(FOnPopulateSKMQuickAccessMenu, FMenuBuilder& /*MenuBuilder*/);

/**
 * Thin Slate host for the Skeletal Mesh Modeling Tools quick-access menu.
 */
class SSkeletalMeshModelingToolsQuickAccessMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSkeletalMeshModelingToolsQuickAccessMenu) {}
		// Should be the toolkit's command list so existing MapAction handlers fire when entries are clicked.
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		SLATE_EVENT(FOnPopulateSKMQuickAccessMenu, OnPopulateMenu)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
