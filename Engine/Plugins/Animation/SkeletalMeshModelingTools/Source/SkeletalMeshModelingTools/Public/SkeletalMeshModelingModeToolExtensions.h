// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "ModelingModeToolExtensions.h"
#include "Internationalization/Text.h"

/**
 * ISkeletalMeshModelingModeToolExtension uses the IModularFeature API to allow a Plugin to provide
 * a set of InteractiveTool's to be exposed in SkeletalMesh Modeling Mode. The Tools will be 
 * included in a section of the SkeletalMesh Modeling Mode tool list, based on GetToolSectionName().
 * 
 */
class ISkeletalMeshModelingModeToolExtension : public IModelingModeToolExtension
{
public:
	virtual ~ISkeletalMeshModelingModeToolExtension() override = default;

	static FName GetModularFeatureName()
	{
		static FName FeatureName( TEXT("SkeletalMeshModelingModeToolExtension") );
		return FeatureName;
	}
};
