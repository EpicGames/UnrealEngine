// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SkeletalMeshModelingTools : ModuleRules
{
	public SkeletalMeshModelingTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CommonMenuExtensions",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"DetailCustomizations",
				"EditorFramework",
				"EditorInteractiveToolsFramework",
				"Engine",
				"InputCore",
				"InteractiveToolsFramework",
				"MeshModelingTools",
				"MeshModelingToolsEditorOnly",
				"ModelingComponentsEditorOnly",
				"ModelingToolsEditorMode",
				"SkeletalMeshEditor",
				"SkeletalMeshUtilitiesCommon",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"StatusBar",
				"PropertyEditor",
				"AnimationCore",
				"AnimationWidgets",
				"ApplicationCore",
				"WidgetRegistration",
				"SkeletalMeshModifiers",
				"Persona",
				"MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"GeometryCore",
				"ModelingComponents",
				"MeshConversion",
				"InterchangeEngine",
				"GeometryFramework",
				"EditorWidgets",
				"DeveloperSettings",
				"DynamicMesh", 
				"MeshModelingToolsEditorOnlyExp",
				"MeshModelingToolsExp",
				"GeometryScriptingEditor",
				"Projects"
			}
		);

		ShortName = "SkelMeshModTools";
	}
}
